// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrowserFeature.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "SandboxedEditingStyle.h"
#include "ViewModels/SandboxControlsViewModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SSandboxedEditingRoot.h"
#include "Widgets/Unsandboxed/List/Columns/DescriptionSandboxColumn.h"
#include "Widgets/Unsandboxed/List/Columns/LastModifiedSandboxColumn.h"
#include "Widgets/Unsandboxed/List/Columns/NameSandboxColumn.h"
#include "Widgets/Unsandboxed/List/Columns/VersionSandboxColumn.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FBrowserFeature"

namespace UE::SandboxedEditing
{
namespace BrowserDetail
{
const FLazyName SandboxedEditingTabName(TEXT("SandboxedEditingTab"));

static void MakeSandboxColumns(
	const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel,
	TMap<FName, TSharedRef<ISandboxColumnBehavior>>& OutBehaviors, TMap<FName, TSharedRef<ISandboxColumnWidgetFactory>>& OutFactories
	)
{
	const TSharedRef<FNameSandboxColumn> Name = MakeShared<FNameSandboxColumn>(InControlsViewModel);
	const TSharedRef<FDescriptionSandboxColumn> Description = MakeShared<FDescriptionSandboxColumn>();
	const TSharedRef<FVersionSandboxColumn> Version = MakeShared<FVersionSandboxColumn>();
	const TSharedRef<FLastModifiedSandboxColumn> LastModified = MakeShared<FLastModifiedSandboxColumn>();
	OutBehaviors = TMap<FName, TSharedRef<ISandboxColumnBehavior>>
	{
		{ NameSandboxColumn, Name },
		{ DescriptionSandboxColumn, Description },
		{ VersionSandboxNameColumn, Version },
		{ LastModifiedSandboxColumn, LastModified }
	};
	OutFactories = TMap<FName, TSharedRef<ISandboxColumnWidgetFactory>>
	{
		{ NameSandboxColumn, Name },
		{ DescriptionSandboxColumn, Description },
		{ VersionSandboxNameColumn, Version },
		{ LastModifiedSandboxColumn, LastModified }
	};
}

static FSandboxColumnRegistry MakeSandboxColumnRegistry(const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel)
{
	FSandboxColumnRegistry Registry;
	MakeSandboxColumns(InControlsViewModel, Registry.ColumnBehaviors, Registry.ColumnFactories);
	return Registry;
}
}
	
FBrowserFeature::FBrowserFeature(
	const TSharedRef<FSandboxSystemModel>& InSandboxModel, 
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& InExternalSandboxViewModel
	)
	: FBrowserFeature(InSandboxModel, InExternalSandboxViewModel, MakeShared<FSandboxControlsViewModel>(InSandboxModel))
{}

FBrowserFeature::FBrowserFeature(
	const TSharedRef<FSandboxSystemModel>& InSandboxModel,
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& InExternalSandboxViewModel,
	const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel
	)
	: SandboxModel(InSandboxModel)
	, SandboxColumns(BrowserDetail::MakeSandboxColumnRegistry(InControlsViewModel))
	, BrowserFileActionColumns(GetColumnsForBrowser())
	, ActiveFileActionColumns(GetColumnsForActiveSandbox())
	, ViewModels(SandboxColumns, BrowserFileActionColumns, ActiveFileActionColumns, SandboxModel, InExternalSandboxViewModel, InControlsViewModel)
	, Views(ViewModels.ControlsViewModel, ViewModels.PersistViewModel, ViewModels.LeaveViewModel)
	, CommandBindings(MakeShared<FBrowserCommandBindings>(ViewModels))
{
	RegisterTabSpawner();
}

FBrowserFeature::~FBrowserFeature()
{
	UnregisterTabSpawner();
}

void FBrowserFeature::SummonUI() const
{
	FGlobalTabmanager::Get()->TryInvokeTab(BrowserDetail::SandboxedEditingTabName.Resolve());
}



void FBrowserFeature::RegisterTabSpawner()
{
	FTabSpawnerEntry& BrowserSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		BrowserDetail::SandboxedEditingTabName, FOnSpawnTab::CreateRaw(this, &FBrowserFeature::SpawnBrowserTab)
		);
	
	BrowserSpawnerEntry
	.SetIcon(FSlateIcon(FSandboxedEditingStyle::Get().GetStyleSetName(), TEXT("SandboxedEditing.Browser.Icon")))
	.SetDisplayName(LOCTEXT("BrowserTabTitle", "Sandboxes"))
	.SetTooltipText(LOCTEXT("BrowserTooltipText", "Open the Sandbox browser"))
	.SetMenuType(ETabSpawnerMenuType::Enabled);
	
	BrowserSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());
}

void FBrowserFeature::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(BrowserDetail::SandboxedEditingTabName);
}

TSharedRef<SDockTab> FBrowserFeature::SpawnBrowserTab(const FSpawnTabArgs& InArgs)
{
	return SNew(SDockTab)
		.TabRole(NomadTab)
		[
			SNew(SSandboxedEditingRoot, ViewModels)
			.CommandList(CommandBindings->GetCommandList())
			.SandboxColumnFactories(SandboxColumns.ColumnFactories)
			.BrowserFileActionsColumnFactories(BrowserFileActionColumns.ColumnFactories)
			.ActiveFileStateColumnsFactories(ActiveFileActionColumns.ColumnFactories)
		];
}
}

#undef LOCTEXT_NAMESPACE