// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/BrowserCommandBindings.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "ViewModels/BrowserViewModels.h"
#include "Views/AskUserAboutDirtyPackagesView.h"
#include "Views/BrowserViews.h"

class SDockTab;
class FSpawnTabArgs;

namespace UE::SandboxedEditing
{
class FActiveSandboxDetailsViewModel;
class FActiveSandboxTrackerViewModel;
class FSandboxControlsViewModel;
class FSandboxListViewModel;
class FSandboxSystemModel;
class SSandboxedEditingRoot;

/** Summons a nomad tab through which the user can browse sandboxes. */
class FBrowserFeature : public FNoncopyable
{
public:
	
	/**
	 * @param InSandboxModel Model that interacts with the core sandbox system.
	 * @param InExternalSandboxViewModel View model to let us know when an external sandbox is active.
	 */
	explicit FBrowserFeature(
		const TSharedRef<FSandboxSystemModel>& InSandboxModel, 
		const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& InExternalSandboxViewModel
		);
	~FBrowserFeature();
	
	/** Summons the browser tab. */
	void SummonUI() const;
	
private:
	
	/** The shared sandbox model. */
	const TSharedRef<FSandboxSystemModel> SandboxModel;
	
	/** The columns that are displayed in the browser. */
	const FSandboxColumnRegistry SandboxColumns;
	/** The columns displayed for the file actions in the browser UI. */
	const FFileStateColumnRegistry BrowserFileActionColumns;
	/** The columns displayed for the file actions in the active sandbox UI. */
	const FFileStateColumnRegistry ActiveFileActionColumns;
	
	/** Holds all view models relevant for the browser. */
	const FBrowserViewModels ViewModels;
	/** Holds all explicit views. */
	const FBrowserViews Views;
	
	/** Binds user commands to models.*/
	const TSharedRef<FBrowserCommandBindings> CommandBindings;
	
	explicit FBrowserFeature(
		const TSharedRef<FSandboxSystemModel>& InSandboxModel, 
		const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& InExternalSandboxViewModel,
		const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel
		);
	
	void RegisterTabSpawner();
	void UnregisterTabSpawner();
	TSharedRef<SDockTab> SpawnBrowserTab(const FSpawnTabArgs& InArgs);
};
}


