// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrowserCommandBindings.h"

#include "BrowserCommands.h"
#include "Features/Browser/ViewModels/BrowserViewModels.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Features/Browser/ViewModels/Active/ActiveSandboxTrackerViewModel.h"
#include "Framework/Commands/GenericCommands.h"

namespace UE::SandboxedEditing
{
FBrowserCommandBindings::FBrowserCommandBindings(const FBrowserViewModels& InViewModels)
	: ControlsViewModel(InViewModels.ControlsViewModel)
	, LeaveSandboxViewModel(InViewModels.LeaveViewModel)
	, PersistViewModel(InViewModels.PersistViewModel)
	, CommandList(MakeShared<FUICommandList>())
{
	FBrowserCommands& BrowserCommands = FBrowserCommands::Get();
	CommandList->MapAction(BrowserCommands.CreateNewSandbox, 
		FExecuteAction::CreateSP(ControlsViewModel, &FSandboxControlsViewModel::StartCreationWorkflow), 
		FCanExecuteAction::CreateSP(ControlsViewModel, &FSandboxControlsViewModel::CanStartCreationWorkflow)
	);
	CommandList->MapAction(BrowserCommands.Cancel, FExecuteAction::CreateRaw(this, &FBrowserCommandBindings::HandleCancel));
	CommandList->MapAction(BrowserCommands.LeaveSandbox, 
		FExecuteAction::CreateRaw(this, &FBrowserCommandBindings::HandleLeaveSandbox),
		FCanExecuteAction::CreateRaw(this, &FBrowserCommandBindings::CanLeaveSandbox)
		);
	CommandList->MapAction(BrowserCommands.PersistSandbox,
		FExecuteAction::CreateRaw(this, &FBrowserCommandBindings::HandlePersistSandbox),
		FCanExecuteAction::CreateRaw(this, &FBrowserCommandBindings::CanPersistSandbox)
		);
}

void FBrowserCommandBindings::HandleCancel() const
{
	ControlsViewModel->CancelWorkflows();
}

void FBrowserCommandBindings::HandleLeaveSandbox()
{
	if (CanLeaveSandbox())
	{
		LeaveSandboxViewModel->LeaveSandbox();
	}
}

bool FBrowserCommandBindings::CanLeaveSandbox()
{
	return LeaveSandboxViewModel->CanLeaveSandbox();
}

void FBrowserCommandBindings::HandlePersistSandbox()
{
	PersistViewModel->StartPersistWorkflowForActiveSandbox();
}

bool FBrowserCommandBindings::CanPersistSandbox()
{
	return PersistViewModel->CanStartPersistWorkflowForActiveSandbox();
}
}
