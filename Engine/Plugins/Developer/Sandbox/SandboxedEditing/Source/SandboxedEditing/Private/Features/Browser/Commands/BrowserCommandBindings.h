// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FActiveSandboxDetailsViewModel;
class FLeaveSandboxViewModel;
class FPersistSandboxViewModel;
class FSandboxControlsViewModel;
struct FBrowserViewModels;

/** Binds the commands relevant for the browser feature */
class FBrowserCommandBindings : public FNoncopyable
{
public:
	
	explicit FBrowserCommandBindings(const FBrowserViewModels& InViewModels);
	
	const TSharedRef<FUICommandList>& GetCommandList() const { return CommandList; }
	
private:
	
	/** Used for editing sandbox info. */
	const TSharedRef<FSandboxControlsViewModel> ControlsViewModel;
	/** Used to interact with the active sandbox. */
	const TSharedRef<FLeaveSandboxViewModel> LeaveSandboxViewModel;
	/** Used to persist the sandbox. */
	const TSharedRef<FPersistSandboxViewModel> PersistViewModel;
	
	/** Holds our command bindings */
	const TSharedRef<FUICommandList> CommandList;
	
	/** Invoked when the delete command is issued. */
	FSimpleMulticastDelegate OnDeleteSelection;
	
	/** Cancels any active operation, such as renaming or creating a sandbox. */
	void HandleCancel() const;
	
	void HandleLeaveSandbox();
	bool CanLeaveSandbox();
	
	void HandlePersistSandbox();
	bool CanPersistSandbox();
};
}

