// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrowserCommands.h"

#include "SandboxedEditingStyle.h"

#define LOCTEXT_NAMESPACE "FBrowserCommands"

namespace UE::SandboxedEditing
{
FBrowserCommands::FBrowserCommands()
	: TCommands(
		TEXT("Browser"), 
		LOCTEXT("Description", "Commands relevant to the sandbox browser"), 
		NAME_None, 
		FSandboxedEditingStyle::Get().GetStyleSetName()
		)
{}

void FBrowserCommands::RegisterCommands()
{
	UI_COMMAND(CreateNewSandbox, "Create New Sandbox", "Creates a new sandbox", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Cancel, "Cancel", "Cancel the current operation, such as renaming or creating a sandbox.", EUserInterfaceActionType::None, FInputChord(EKeys::Escape));
	
	UI_COMMAND(LeaveSandbox, "Leave Sandbox", "Leaves the current sandbox.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PersistSandbox, "Persist Sandbox", "Persists the active sandbox.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ExportSandboxes, "Export", "Exports the sandboxes selected in the browser as zip archive.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportSandboxes, "Import", "Imports sandboxes saved in zip archives.", EUserInterfaceActionType::Button, FInputChord());
}
}

#undef LOCTEXT_NAMESPACE