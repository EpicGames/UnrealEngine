// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::SandboxedEditing
{
/** Commands used in the browser feature. */
class FBrowserCommands : public TCommands<FBrowserCommands>
{
public:
	
	FBrowserCommands();
	
	//~ Begin TCommands Interface
	virtual void RegisterCommands() override;
	//~ End TCommands Interface
	
	/** Starts the workflow for creating a new sandbox. */
	TSharedPtr<FUICommandInfo> CreateNewSandbox;
	
	/** Cancel the current operation, such as renaming or creating a sandbox. */
	TSharedPtr<FUICommandInfo> Cancel;
	
	/** Leaves the current sandbox. */
	TSharedPtr<FUICommandInfo> LeaveSandbox;
	/** Persists the current sandbox (without leaving). */
	TSharedPtr<FUICommandInfo> PersistSandbox;
	
	/** Exports the sandboxes selected in the browser. */
	TSharedPtr<FUICommandInfo> ExportSandboxes;
	/** Imports sandboxes selected from the system explorer. */
	TSharedPtr<FUICommandInfo> ImportSandboxes;
};
}

