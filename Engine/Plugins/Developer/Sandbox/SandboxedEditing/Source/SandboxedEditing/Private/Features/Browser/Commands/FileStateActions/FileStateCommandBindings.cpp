// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileStateCommandBindings.h"

#include "ContentBrowserModule.h"
#include "FileStateCommands.h"
#include "IContentBrowserSingleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Features/Browser/Commands/BrowserCommands.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Features/Browser/Views/Menu/FileStateMenuUtils.h"
#include "Features/Browser/Widgets/Shared/FileState/SFileStateListView.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/PackageName.h"

namespace UE::SandboxedEditing
{
namespace CommonMenuDetail
{
static TArray<FString> TransformSelectedItems(TSharedPtr<SFileStateListView> InListView)
{
	const TArray<TSharedPtr<FFileStateItem>> SelectedItems = InListView->GetSelectedFiles();
	TArray<FString> NonSandboxFiles;
	Algo::Transform(SelectedItems, NonSandboxFiles, [](const TSharedPtr<FFileStateItem>& InItem)
	{
		return InItem->NonSandboxFile;
	});
	return NonSandboxFiles;
}
}

FFileStateCommandBindings::FFileStateCommandBindings(TAttribute<TOptional<FString>> InSelectedSandboxRootAttr, TAttribute<TArray<FString>> InSelectedFilePathsAttr)
	: SelectedSandboxRootAttr(MoveTemp(InSelectedSandboxRootAttr))
	, SelectedFilePathsAttr(MoveTemp(InSelectedFilePathsAttr))
{
	BindCommands();
}

void FFileStateCommandBindings::AppendMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.PushCommandList(CommandList);
	FileStateMenu::AppendMenu(InMenuBuilder);
	InMenuBuilder.PopCommandList();
}

void FFileStateCommandBindings::BindCommands()
{
	FFileStateCommands Commands = FFileStateCommands::Get();
	
	CommandList->MapAction(
		Commands.BrowseToAsset, 
		FExecuteAction::CreateRaw(this, &FFileStateCommandBindings::HandleBrowseToAsset), 
		FCanExecuteAction::CreateRaw(this, &FFileStateCommandBindings::CanBrowseToAsset)
		);
	CommandList->MapAction(
		Commands.ShowRootInExplorer, FExecuteAction::CreateRaw(this, &FFileStateCommandBindings::HandleShowRootInExplorer)
		);
	CommandList->MapAction(
		Commands.ShowFileInExplorer, 
		FExecuteAction::CreateRaw(this, &FFileStateCommandBindings::HandleShowFileInExplorer),
		FCanExecuteAction::CreateRaw(this, &FFileStateCommandBindings::CanShowFileInExplorer)
		);
}

namespace CommandDetails
{
static TArray<FAssetData> GetAssetDataFromPaths(const TArray<FString>& InAbsolutePaths)                                                                                                                                                                                                                          
{                                                                                                                                                                                                                                                                                                                
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();                                                                                                                                                                           
	
	TArray<FAssetData> Result;                                                                                                                                                                                                                                                                                   
	Result.Reserve(InAbsolutePaths.Num());                                                                                                                                                                                                                                                                       
	
	for (const FString& AbsolutePath : InAbsolutePaths)
	{
		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(AbsolutePath, PackageName))
		{
			AssetRegistry.GetAssetsByPackageName(FName(*PackageName), Result);
		}
  }

  return Result;
}
}

void FFileStateCommandBindings::HandleBrowseToAsset() const
{
	const TArray<FAssetData> Assets = CommandDetails::GetAssetDataFromPaths(SelectedFilePathsAttr.Get());
	if (!Assets.IsEmpty())
	{
		const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
	}
}

bool FFileStateCommandBindings::CanBrowseToAsset() const
{
	const TArray<FAssetData> Assets = CommandDetails::GetAssetDataFromPaths(SelectedFilePathsAttr.Get());
	return !Assets.IsEmpty();
}

void FFileStateCommandBindings::HandleShowRootInExplorer() const
{
	if (const TOptional<FString> SandboxRoot = SelectedSandboxRootAttr.Get())
	{
		FPlatformProcess::ExploreFolder(**SandboxRoot);
	}
}

void FFileStateCommandBindings::HandleShowFileInExplorer() const
{
	const TOptional<FString> SandboxRoot = SelectedSandboxRootAttr.Get();
	const TArray<FString> SelectedPaths = SelectedFilePathsAttr.Get();
	if (!SandboxRoot || SelectedPaths.Num() != 1)
	{
		return;
	}
	
	if (const TOptional<FString> NonSandboxPath = SandboxModel::GetSandboxPathFor(*SandboxRoot, SelectedPaths[0]))
	{
		FPlatformProcess::ExploreFolder(**NonSandboxPath);
	}
}

bool FFileStateCommandBindings::CanShowFileInExplorer() const
{
	const TOptional<FString> SandboxRoot = SelectedSandboxRootAttr.Get();
	const TArray<FString> SelectedPaths = SelectedFilePathsAttr.Get();
	if (!SandboxRoot || SelectedPaths.Num() != 1)
	{
		return false;
	}
	
	return SandboxModel::GetSandboxPathFor(*SandboxRoot, SelectedPaths[0]).IsSet();
}
}
