// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FMenuBuilder;
class FUICommandList;

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
class SFileStateListView;

/** 
 * Builds the right-click menu items displayed in all instances of SFileStateListView.
 * Binds & unbinds the relevant commands required for that menu.
 */
class FFileStateCommandBindings : public FNoncopyable
{
public:
	
	explicit FFileStateCommandBindings(
		TAttribute<TOptional<FString>> InSelectedSandboxRootAttr, TAttribute<TArray<FString>> InSelectedFilePathsAttr
		);

	/** Utility to append the commands to a menu builder. */
	void AppendMenu(FMenuBuilder& InMenuBuilder);
	
private:
	
	/** The command list that our commands are bound to. */
	const TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();
	 	
	/** The sandbox root path on which the commands should be run. */
	const TAttribute<TOptional<FString>> SelectedSandboxRootAttr;
	/** Gets the items selected in the UI. */
	const TAttribute<TArray<FString>> SelectedFilePathsAttr;

	/** Binds the commands needed for the menu. */
	void BindCommands();
	
	void HandleBrowseToAsset() const;
	bool CanBrowseToAsset() const;
	
	void HandleShowRootInExplorer() const;
	
	void HandleShowFileInExplorer() const;
	bool CanShowFileInExplorer() const;
};
}


