// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileStateCommands.h"

#include "SandboxedEditingStyle.h"

#define LOCTEXT_NAMESPACE "FFileStateCommands"

namespace UE::SandboxedEditing
{
FFileStateCommands::FFileStateCommands()
	: TCommands(
		TEXT("FileState"), 
		LOCTEXT("Description", "Commands relevant to the sandbox browser"), 
		TEXT("Browser"), 
		FSandboxedEditingStyle::Get().GetStyleSetName()
	)
{}

void FFileStateCommands::RegisterCommands()
{
	UI_COMMAND(BrowseToAsset, "Browse to asset", "Shows this asset in the content browser.", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND(ShowRootInExplorer, "Show root sandbox folder", "Shows sandbox root directory in the system file manager.", EUserInterfaceActionType::Button, FInputChord())
	UI_COMMAND(ShowFileInExplorer, "Show sandbox file", "Shows the file's sandboxed counterpart in the system file manager.", EUserInterfaceActionType::Button, FInputChord())
	
	UI_COMMAND(PersistSelected, "Persist", "Open the persist dialogue with the currently selected files already pre-selected in the dialogue.", EUserInterfaceActionType::Button, FInputChord())
}
}

#undef LOCTEXT_NAMESPACE
