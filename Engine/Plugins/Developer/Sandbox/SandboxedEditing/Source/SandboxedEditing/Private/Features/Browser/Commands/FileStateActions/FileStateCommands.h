// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::SandboxedEditing
{
/** Commands used for actions performed on sandboxed files (usually on selected items in SFileStateListView). */
class FFileStateCommands : public TCommands<FFileStateCommands>
{
public:
	
	FFileStateCommands();
	
	//~ Begin TCommands Interface
	virtual void RegisterCommands() override;
	//~ End TCommands Interface
	
	/** Opens the content browser to this file*/
	TSharedPtr<FUICommandInfo> BrowseToAsset;
	/** Shows sandbox root directory in the system file explorer. */
	TSharedPtr<FUICommandInfo> ShowRootInExplorer;
	/** Shows the file's sandboxed counterpart in the system file explorer. */
	TSharedPtr<FUICommandInfo> ShowFileInExplorer;
	
	/** Persists the selected files. */
	TSharedPtr<FUICommandInfo> PersistSelected;
};
}

