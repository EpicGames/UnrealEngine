// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveSandboxDetailsViewModel.h"

#include "Features/Browser/ViewModels/Leaving/LeaveSandboxViewModel.h"
#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FActiveSandboxDetailsViewModel::FActiveSandboxDetailsViewModel(
	const TSharedRef<FSandboxSystemModel>& InModel, 
	const TSharedRef<FLeaveSandboxViewModel>& InLeaveSandboxViewModel
	)
	: Model(InModel)
	, LeaveSandboxViewModel(InLeaveSandboxViewModel)
{}

void FActiveSandboxDetailsViewModel::LeaveSandbox()
{
	LeaveSandboxViewModel->LeaveSandbox();
}

bool FActiveSandboxDetailsViewModel::CanLeaveSandbox(FText* OutReason) const
{
	return LeaveSandboxViewModel->CanLeaveSandbox(OutReason);
}

FString FActiveSandboxDetailsViewModel::GetSandboxName() const
{
	return Model->GetActiveSandboxName();
}

FString FActiveSandboxDetailsViewModel::GetSandboxPath() const
{
	return Model->GetActiveSandboxPath();
}
}
