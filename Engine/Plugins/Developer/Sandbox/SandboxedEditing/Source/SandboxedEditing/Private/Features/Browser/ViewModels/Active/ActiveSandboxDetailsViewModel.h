// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FLeaveSandboxViewModel;
class FSandboxSystemModel;

/** Information about the active sandbox. */
class FActiveSandboxDetailsViewModel : public FNoncopyable
{
public:
	
	explicit FActiveSandboxDetailsViewModel(
		const TSharedRef<FSandboxSystemModel>& InModel, 
		const TSharedRef<FLeaveSandboxViewModel>& InLeaveSandboxViewModel
		);
	
	/** Leaves the active sandbox. */
	void LeaveSandbox();
	/** @return Whether the sandbox can be left */
	bool CanLeaveSandbox(FText* OutReason = nullptr) const; 
	
	/** @return Name of the active sandbox. */
	FString GetSandboxName() const;
	/** @return Path to active sandbox root. */
	FString GetSandboxPath() const;
	
private:
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** Handles leave logic. */
	const TSharedRef<FLeaveSandboxViewModel> LeaveSandboxViewModel;
};
}

