// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveSandboxTrackerViewModel.h"

#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FActiveSandboxTrackerViewModel::FActiveSandboxTrackerViewModel(const TSharedRef<FSandboxSystemModel>& InModel)
	: Model(InModel)
{
	Model->OnLoadSandbox().AddRaw(this, &FActiveSandboxTrackerViewModel::HandleSandboxedLoaded);
	Model->OnLeaveSandbox().AddRaw(this, &FActiveSandboxTrackerViewModel::HandleSandboxLeft);
}

FActiveSandboxTrackerViewModel::~FActiveSandboxTrackerViewModel()
{
	Model->OnLoadSandbox().RemoveAll(this);
	Model->OnLeaveSandbox().RemoveAll(this);
}

bool FActiveSandboxTrackerViewModel::HasActiveSandbox() const
{
	return Model->HasActiveSandbox();
}

void FActiveSandboxTrackerViewModel::HandleSandboxedLoaded()
{
	OnLoadSandboxDelegate.Broadcast();
}

void FActiveSandboxTrackerViewModel::HandleSandboxLeft()
{
	OnLeaveSandboxDelegate.Broadcast();
}
}
