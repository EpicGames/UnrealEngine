// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActiveSandboxDetailsViewModel.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** Knows when the active sandbox changes. */
class FActiveSandboxTrackerViewModel : public FNoncopyable
{
public:
	
	explicit FActiveSandboxTrackerViewModel(const TSharedRef<FSandboxSystemModel>& InModel);
	~FActiveSandboxTrackerViewModel();
	
	/** @return Whether there is an active sandbox */
	bool HasActiveSandbox() const;
	
	/** Invoked when the engine has entered an active sandbox. */
	FSimpleMulticastDelegate& OnLoadSandbox() { return OnLoadSandboxDelegate; }
	/** Invoked when the active has stopped being in an active sandbox. */
	FSimpleMulticastDelegate& OnLeaveSandbox() { return OnLeaveSandboxDelegate; }
	
private:
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** Invoked when a new sandbox is loaded. */
	FSimpleMulticastDelegate OnLoadSandboxDelegate;
	/** Invoked when the current sandbox is left. */
	FSimpleMulticastDelegate OnLeaveSandboxDelegate;
	
	/** Creates the view model and invokes OnLoadSandboxDelegate. */
	void HandleSandboxedLoaded();
	/** Cleans up the view model and invokes OnLeaveSandboxDelegate. */
	void HandleSandboxLeft();
};
}

