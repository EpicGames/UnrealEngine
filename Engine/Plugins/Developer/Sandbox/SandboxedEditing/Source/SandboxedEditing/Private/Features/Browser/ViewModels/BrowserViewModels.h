// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Active/ActiveSandboxDetailsViewModel.h"
#include "Active/ActiveSandboxTrackerViewModel.h"
#include "FileState/FileStateColumnRegistry.h"
#include "FileState/Models/ActiveSandboxFileStateViewModel.h"
#include "FileState/Models/UnloadedSandboxFileStateViewModel.h"
#include "Leaving/LeaveSandboxViewModel.h"
#include "List/FilterSandboxViewModel.h"
#include "List/SandboxColumnRegistry.h"
#include "List/SandboxListViewModel.h"
#include "Persist/PersistSandboxViewModel.h"
#include "SandboxControlsViewModel.h"
#include "SandboxMetaDataViewModel.h"
#include "FileState/FilterFileStateViewModel.h"
#include "Templates/SharedPointer.h"

namespace UE::FileSandboxUI { class IExternalSandboxActiveViewModel; }

namespace UE::SandboxedEditing
{
class ISandboxColumnBehavior;

/** Holds all view models used by the root UI. Make sharing view models easier. */
struct FBrowserViewModels
{
	/** Knows how to filter sandbox items. */
	const TSharedRef<FFilterSandboxViewModel> FilterSandboxViewModel;
	/** Knows about the items that are displayed in the browser. */
	const TSharedRef<FSandboxListViewModel> ListViewModel;
	
	
	/** The filters for UnloadedSandboxFileStateViewModel */
	const TSharedRef<FFilterFileStateViewModel> FilterUnloadedSandboxFileStateViewModel;
	/** Knows the file action of the currently selected sandbox. */
	const TSharedRef<FUnloadedSandboxFileStateViewModel> UnloadedSandboxFileStateViewModel;
	
	/** The filters for ActiveSandboxFileStateViewModel */
	const TSharedRef<FFilterFileStateViewModel> FilterActiveSandboxFileStateViewModel;
	/** Knows the file actions of the active sandbox. */
	const TSharedRef<FActiveSandboxFileStateViewModel> ActiveSandboxFileStateViewModel;
	
	
	/** Handles actions that the user performs on the sandboxes via the UI. */
	const TSharedRef<FSandboxControlsViewModel> ControlsViewModel;
	/** Handles changing metadata of a sandbox. */
	const TSharedRef<FSandboxMetaDataViewModel> MetaDataViewModel;
	
	
	/** Handles persisting the active sandbox. */
	const TSharedRef<FPersistSandboxViewModel> PersistViewModel;
	/** Handles leaving an active sandbox. */
	const TSharedRef<FLeaveSandboxViewModel> LeaveViewModel;
	
	
	/** Used for widgets to switch states when the active sandbox changes. */
	const TSharedRef<FActiveSandboxTrackerViewModel> ActiveSandboxTrackerViewModel;
	/** Used for widgets that display info about the currently active sandbox. */
	const TSharedRef<FActiveSandboxDetailsViewModel> ActiveSandboxDetailsViewModel;
	/** Knows about when an extern sandbox is being displayed. */
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;
	
	
	explicit FBrowserViewModels(
		const FSandboxColumnRegistry& InSandboxColumns, 
		const FFileStateColumnRegistry& InBrowserFileActionColumns,
		const FFileStateColumnRegistry& InActiveFileActionColumns,
		const TSharedRef<FSandboxSystemModel>& InModel,
		const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& InExternalSandboxViewModel,
		const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel
		)
		: FilterSandboxViewModel(MakeShared<FFilterSandboxViewModel>(InSandboxColumns.GetBehaviorArray()))
		, ListViewModel(MakeShared<FSandboxListViewModel>(InModel, FilterSandboxViewModel, InSandboxColumns.ColumnBehaviors))
		, FilterUnloadedSandboxFileStateViewModel(MakeShared<FFilterFileStateViewModel>(InBrowserFileActionColumns.ToBehaviorArray()))
		, UnloadedSandboxFileStateViewModel(MakeShared<FUnloadedSandboxFileStateViewModel>(InModel, FilterUnloadedSandboxFileStateViewModel, InBrowserFileActionColumns.ColumnBehaviors))
		, FilterActiveSandboxFileStateViewModel(MakeShared<FFilterFileStateViewModel>(InActiveFileActionColumns.ToBehaviorArray()))
		, ActiveSandboxFileStateViewModel(MakeShared<FActiveSandboxFileStateViewModel>(InModel, FilterActiveSandboxFileStateViewModel, InActiveFileActionColumns.ColumnBehaviors))
		, ControlsViewModel(InControlsViewModel)
		, MetaDataViewModel(MakeShared<FSandboxMetaDataViewModel>(InModel))
		, PersistViewModel(MakeShared<FPersistSandboxViewModel>(InModel))
		, LeaveViewModel(MakeShared<FLeaveSandboxViewModel>(InModel, PersistViewModel))
		, ActiveSandboxTrackerViewModel(MakeShared<FActiveSandboxTrackerViewModel>(InModel))
		, ActiveSandboxDetailsViewModel(MakeShared<FActiveSandboxDetailsViewModel>(InModel, LeaveViewModel))
		, ExternalSandboxViewModel(InExternalSandboxViewModel)
	{}
};
}
