// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
/** Interface for a column that can be searched */
template<typename TRowData>
class ISearchableColumnBehavior
{
public:

	/** Search terms for this column */
	virtual void PopulateSearchTerms(const TRowData& InRowData, TArray<FString>& OutSearchTerms) const = 0;

	virtual ~ISearchableColumnBehavior() = default;
};
}
