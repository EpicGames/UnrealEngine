// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
/** Interface for a column that can be sorted */
template<typename TRowData>
class ISortableColumnBehavior
{
public:

	/** @return Whether this column supports sorting */
	virtual bool CanSort() const = 0;

	/**
	 * Compares two row items for sorting.
	 * @param Lhs The left-hand side item
	 * @param Rhs The right-hand side item
	 * @param SortMode The sort mode (Ascending or Descending)
	 * @return true if Lhs should come before Rhs, false otherwise
	 */
	virtual bool SortBy(const TRowData& Lhs, const TRowData& Rhs, EColumnSortMode::Type SortMode) const = 0;

	virtual ~ISortableColumnBehavior() = default;
};
}
