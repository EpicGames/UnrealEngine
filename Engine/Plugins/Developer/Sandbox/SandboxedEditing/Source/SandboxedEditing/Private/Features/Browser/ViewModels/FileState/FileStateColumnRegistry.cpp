// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileStateColumnRegistry.h"

#include "FileStateColumns.h"
#include "Features/Browser/Widgets/Shared/FileState/Columns/ActionFileStateColumn.h"
#include "Features/Browser/Widgets/Shared/FileState/Columns/PathFileStateColumn.h"
#include "Features/Browser/Widgets/Shared/FileState/Columns/PersistFileStateColumn.h"
#include "Features/Browser/Widgets/Shared/FileState/Columns/TimestampFileStateColumn.h"

namespace UE::SandboxedEditing
{
class FPathFileStateColumn;

FFileStateColumnRegistry GetColumnsForBrowser()
{
	const TSharedRef<FActionFileStateColumn> ActionColum = MakeShared<FActionFileStateColumn>();
	const TSharedRef<FTimestampFileStateColumn> Timestamp = MakeShared<FTimestampFileStateColumn>();
	const TSharedRef<FPathFileStateColumn> PathColumn = MakeShared<FPathFileStateColumn>();
	
	FFileStateColumnRegistry Registry;
	Registry.ColumnBehaviors = TMap<FName, TSharedRef<IFileStateColumnBehavior>>
	{
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn }
	};
	Registry.ColumnFactories = TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>>
	{
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn }
	};
	return Registry;
}

FFileStateColumnRegistry GetColumnsForActiveSandbox()
{
	const TSharedRef<FActionFileStateColumn> ActionColum = MakeShared<FActionFileStateColumn>();
	const TSharedRef<FTimestampFileStateColumn> Timestamp = MakeShared<FTimestampFileStateColumn>();
	const TSharedRef<FPathFileStateColumn> PathColumn = MakeShared<FPathFileStateColumn>();
	
	FFileStateColumnRegistry Registry;
	Registry.ColumnBehaviors = TMap<FName, TSharedRef<IFileStateColumnBehavior>>
	{
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn },
	};
	Registry.ColumnFactories = TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>>
	{
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn },
	};
	return Registry;
}

FFileStateColumnRegistry GetColumnsForPersist(const TSharedRef<FPersistOperationViewModel>& InViewModel)
{
	const TSharedRef<FPersistFileStateColumn> CheckboxColumn = MakeShared<FPersistFileStateColumn>(InViewModel);
	const TSharedRef<FActionFileStateColumn> ActionColum = MakeShared<FActionFileStateColumn>();
	const TSharedRef<FTimestampFileStateColumn> Timestamp = MakeShared<FTimestampFileStateColumn>();
	const TSharedRef<FPathFileStateColumn> PathColumn = MakeShared<FPathFileStateColumn>();
	
	FFileStateColumnRegistry Registry;
	Registry.ColumnBehaviors = TMap<FName, TSharedRef<IFileStateColumnBehavior>>
	{
		{ PersistCheckboxColumn, CheckboxColumn },
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn },
	};
	Registry.ColumnFactories = TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>>
	{
		{ PersistCheckboxColumn, CheckboxColumn },
		{ FileActionColumn, ActionColum },
		{ FileTimestampColumn, Timestamp },
		{ FilePathColumn, PathColumn },
	};
	return Registry;
}
}
