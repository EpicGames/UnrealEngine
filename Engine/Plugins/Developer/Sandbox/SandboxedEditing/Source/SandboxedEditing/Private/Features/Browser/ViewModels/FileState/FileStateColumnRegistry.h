// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

namespace UE::SandboxedEditing
{
class FPersistOperationViewModel;
class IFileStateColumnBehavior;
class IFileStateColumnWidgetFactory;

struct FFileStateColumnRegistry
{
	/** The behavior of the columns for view models. */
	TMap<FName, TSharedRef<IFileStateColumnBehavior>> ColumnBehaviors;
	
	/** Creates the widgets for the columns. */
	TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>> ColumnFactories;
	
	TArray<TSharedRef<IFileStateColumnBehavior>> ToBehaviorArray() const
	{
		TArray<TSharedRef<IFileStateColumnBehavior>> Behaviors;
		ColumnBehaviors.GenerateValueArray(Behaviors);
		return Behaviors;
	}
};
	
/** @return Columns to show in the browser view. */
FFileStateColumnRegistry GetColumnsForBrowser();

/** @return Columns to show in the active sandbox view. */
FFileStateColumnRegistry GetColumnsForActiveSandbox();

/** @return Columns to show in the persist view. */
FFileStateColumnRegistry GetColumnsForPersist(const TSharedRef<FPersistOperationViewModel>& InViewModel);
}