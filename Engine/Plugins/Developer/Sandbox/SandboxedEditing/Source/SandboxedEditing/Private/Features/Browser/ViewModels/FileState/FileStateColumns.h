// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::SandboxedEditing
{
class IFileStateColumnBehavior;
class IFileStateColumnWidgetFactory;
struct FFileStateItem;

/** Needed for SLATE_ARGUMENT (the macro is written in such a way that TMap is not parsed correctly). */
using FFileStateColumnFactoryMap = TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>>;

constexpr FLazyName PersistCheckboxColumn(TEXT("PersistCheckboxColumn"));
constexpr FLazyName FileActionColumn(TEXT("FileActionColumn"));
constexpr FLazyName FilePathColumn(TEXT("FilePathColumn"));
constexpr FLazyName FileTimestampColumn(TEXT("FileTimestampColumn"));
}
