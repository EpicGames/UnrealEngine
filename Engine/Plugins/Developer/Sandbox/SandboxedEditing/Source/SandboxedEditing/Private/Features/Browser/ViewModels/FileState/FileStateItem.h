// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/DateTime.h"
#include "Types/SandboxFileChange.h"

namespace UE::SandboxedEditing
{
/** Represents an item displayed in IFileStateViewModel. */
struct FFileStateItem
{
	/** Path to the non-sandboxed file. */
	FString NonSandboxFile;
	
	/** The operation performed on the file. */
	FileSandboxCore::ESandboxFileChange Action;
	
	/** The time of the edit */
	FDateTime Timestamp;

	explicit FFileStateItem(FString InNonSandboxFile, FileSandboxCore::ESandboxFileChange InAction, const FDateTime& InTimestamp)
		: NonSandboxFile(MoveTemp(InNonSandboxFile))
		, Action(InAction)
		, Timestamp(InTimestamp)
	{}
	
	/** @return Whether the timestamp is valid. FileSandboxCore API returns FDateTime::MinValue for invalid timestamps. */
	bool HasValidTimestamp() const { return Timestamp != FDateTime::MinValue(); }
};
}
