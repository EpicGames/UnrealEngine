// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileStateItemUtils.h"

#include "FileStateItem.h"
#include "Templates/SharedPointer.h"
#include "Types/GatheredFileChanges.h"
#include "Types/SandboxFileChange.h"

namespace UE::SandboxedEditing
{
TArray<TSharedPtr<FFileStateItem>> MakeItemsFromFileChanges(const FileSandboxCore::FGatheredFileChanges& InChanges)
{
	const TArray<FString>& NonSandboxPaths = InChanges.NonSandboxPaths;
	const TArray<FileSandboxCore::ESandboxFileChange>& FileActions = InChanges.FileActions;
	const TArray<FDateTime>& Timestamps = InChanges.Timestamps;
	check(NonSandboxPaths.Num() == FileActions.Num() && NonSandboxPaths.Num() == Timestamps.Num());
	
	TArray<TSharedPtr<FFileStateItem>> Items;
	for (int32 Index = 0; Index < NonSandboxPaths.Num(); ++Index)
	{
		Items.Emplace(MakeShared<FFileStateItem>(NonSandboxPaths[Index], FileActions[Index], Timestamps[Index]));
	}
	return Items;
}
}
