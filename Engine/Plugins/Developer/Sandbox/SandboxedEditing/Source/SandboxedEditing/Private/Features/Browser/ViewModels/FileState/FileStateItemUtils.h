// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::FileSandboxCore { struct FGatheredFileChanges; }

namespace UE::SandboxedEditing
{
struct FFileStateItem;

/** @return An item array transformed from FGatheredFileChanges  */
TArray<TSharedPtr<FFileStateItem>> MakeItemsFromFileChanges(const FileSandboxCore::FGatheredFileChanges& InChanges);
}

