// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterFileStateViewModel.h"

#include "IFileStateColumnBehavior.h"

namespace UE::SandboxedEditing
{
FFilterFileStateViewModel::FFilterFileStateViewModel(TAttribute<TArray<TSharedRef<IFileStateColumnBehavior>>> InVisibleColumns)
	: VisibleColumns(MoveTemp(InVisibleColumns))
	, TextFilter(FFileActionTextFilter::FItemToStringArray::CreateRaw(this, &FFilterFileStateViewModel::ItemToString))
{
	TextFilter.OnChanged().AddLambda([this]
	{
		OnFilterChangedDelegate.Broadcast();
	});
}

FText FFilterFileStateViewModel::GetSearchText() const
{
	return TextFilter.GetRawFilterText();
}

void FFilterFileStateViewModel::SetSearchText(const FText& InSearchText)
{
	TextFilter.SetRawFilterText(InSearchText);
}

bool FFilterFileStateViewModel::PassesFilter(const TSharedRef<FFileStateItem>& InItem) const
{
	return TextFilter.PassesFilter(InItem);
}

bool FFilterFileStateViewModel::AreAnyFiltersActive() const
{
	return !GetSearchText().IsEmpty();
}

void FFilterFileStateViewModel::ItemToString(const TSharedRef<FFileStateItem>& InItem, TArray<FString>& OutSearchTerms)
{
	for (const TSharedRef<IFileStateColumnBehavior>& Column : VisibleColumns.Get())
	{
		Column->PopulateSearchTerms(InItem, OutSearchTerms);
	}
}
}
