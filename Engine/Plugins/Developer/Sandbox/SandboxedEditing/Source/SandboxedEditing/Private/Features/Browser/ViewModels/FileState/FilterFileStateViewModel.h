// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Misc/TextFilter.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class IFileStateColumnBehavior;
struct FFileStateItem;

/** Knows of all filters for filtering file state items. */
class FFilterFileStateViewModel : public FNoncopyable
{
	using FFileActionTextFilter = TTextFilter<const TSharedRef<FFileStateItem>&>;
public:
	
	explicit FFilterFileStateViewModel(TAttribute<TArray<TSharedRef<IFileStateColumnBehavior>>> InVisibleColumns);
	
	/** @return Current search text */
	FText GetSearchText() const;
	/** Sets the search text. Will trigger a refilter. */
	void SetSearchText(const FText& InSearchText);
	
	/** @return Whether the item can be displayed. */
	bool PassesFilter(const TSharedRef<FFileStateItem>& InItem) const;
	
	/** @return Whether any filters are active. */
	bool AreAnyFiltersActive() const;
	
	/** @return Delegate invoked when any of the filters changes */
	FSimpleMulticastDelegate& OnFilterChanged() { return OnFilterChangedDelegate; }
	
private:
	
	/** Visible columns. Used for filtering visible columns by text. */
	const TAttribute<TArray<TSharedRef<IFileStateColumnBehavior>>> VisibleColumns;
	
	/** Does text filltering on items */
	FFileActionTextFilter TextFilter;
	
	/** Invoked when any of the filters changes */
	FSimpleMulticastDelegate OnFilterChangedDelegate;
	
	/** Converts an item to string. */
	void ItemToString(const TSharedRef<FFileStateItem>& InItem, TArray<FString>& OutSearchTerms);
};
}

