// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/Columns/ISearchableColumnBehavior.h"
#include "Features/Browser/ViewModels/Columns/ISortableColumnBehavior.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::SandboxedEditing
{
struct FFileStateItem;

/**
 * View-model logic that is implemented by columns.
 * Supports both searching and sorting.
 */
class IFileStateColumnBehavior
	: public ISearchableColumnBehavior<TSharedPtr<FFileStateItem>>
	, public ISortableColumnBehavior<TSharedPtr<FFileStateItem>>
{};
}
