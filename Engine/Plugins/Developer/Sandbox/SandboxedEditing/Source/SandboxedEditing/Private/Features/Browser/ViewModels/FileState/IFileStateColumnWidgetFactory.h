// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
struct FFileStateItem;

/** Args for making a column widget in a row. */
struct FMakeFileStateColumnWidgetArgs
{
	/** The row data for this column widget */
	TSharedPtr<FFileStateItem> RowData;
	
	/** The (search) text to highlight in widgets. */
	TAttribute<FText> HighlightText;

	explicit FMakeFileStateColumnWidgetArgs(TSharedPtr<FFileStateItem> InRowData, TAttribute<FText> InHighlightText)
		: RowData(MoveTemp(InRowData))
		, HighlightText(MoveTemp(InHighlightText))
	{}
};

/** Interface for creating Slate widgets for a column. */
class IFileStateColumnWidgetFactory
{
public:
	
	/** @return Info required to construct the column. */
	virtual SHeaderRow::FColumn::FArguments MakeColumnArguments() = 0;
	
	/** @return Widget that should be put into the row for this column */
	virtual TSharedRef<SWidget> MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs) = 0;

	virtual ~IFileStateColumnWidgetFactory() = default;
};
}
