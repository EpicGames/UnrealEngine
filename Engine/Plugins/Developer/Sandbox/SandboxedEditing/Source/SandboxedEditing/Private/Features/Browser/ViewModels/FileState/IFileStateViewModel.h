// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
struct FFileStateItem;

/**
 * View model for UI that shows the files added / removed / edited in the sandbox.
 *
 * The implementation can vary based on where the UI is displayed, e.g.
 * - Persist files (part of workflow for leaving sandbox)
 * - Sandbox changed files (part of UI for displaying current sandbox details)
 */
class IFileStateViewModel
{
public:

	/** @return The items to display */
	virtual const TArray<TSharedPtr<FFileStateItem>>& GetItems() const = 0;

	/** @return The columns that should be displayed. */
	virtual TArray<FName> GetDisplayedColumns() const = 0;

	/** Invoked when the items change. */
	virtual FSimpleMulticastDelegate& OnItemsChanged() = 0;

	/**
	 * Returns the current column sort mode (ascending or descending) if the ColumnId parameter matches the current
	 * column to be sorted by, otherwise returns EColumnSortMode::None.
	 *
	 * @param ColumnId Column ID to query sort mode for.
	 * @return The sort mode for the column, or EColumnSortMode::None if it is not known.
	 */
	virtual EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const = 0;

	/**
	 * Returns the current column sort priority (primary or secondary) if the ColumnId parameter matches a sorted column,
	 * otherwise returns EColumnSortPriority::Max.
	 *
	 * @param ColumnId Column ID to query sort priority for.
	 * @return The sort priority for the column, or EColumnSortPriority::Max if it is not sorted.
	 */
	virtual EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const = 0;

	/**
	 * Callback for SHeaderRow::Column::OnSort, called when the column to sort by is changed.
	 *
	 * @param SortPriority The sort priority (Primary or Secondary)
	 * @param ColumnId The column to sort by
	 * @param InSortMode The sort mode (ascending or descending)
	 */
	virtual void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode) = 0;

	virtual ~IFileStateViewModel() = default;
};
}
