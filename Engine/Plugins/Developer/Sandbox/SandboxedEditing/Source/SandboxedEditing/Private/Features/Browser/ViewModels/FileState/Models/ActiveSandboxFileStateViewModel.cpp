// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveSandboxFileStateViewModel.h"

#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Misc/Paths.h"
#include "Types/EBreakBehavior.h"
#include "Types/SandboxedFileChangeInfo.h"

namespace UE::SandboxedEditing
{
FActiveSandboxFileStateViewModel::FActiveSandboxFileStateViewModel(
	const TSharedRef<FSandboxSystemModel>& InModel,
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel,
	TMap<FName, TSharedRef<IFileStateColumnBehavior>> InColumns
	)
	: FStaticFileStateViewModel(MoveTemp(InColumns), {}, InFilterViewModel)
	, Model(InModel)
{
	Model->OnLoadSandbox().AddRaw(this, &FActiveSandboxFileStateViewModel::OnSandboxLoaded);
	Model->OnLeaveSandbox().AddRaw(this, &FActiveSandboxFileStateViewModel::OnSandboxLeft);
	Model->OnSandboxFilesChanged().AddRaw(this, &FActiveSandboxFileStateViewModel::RefreshItems);
	
	if (Model->HasActiveSandbox())
	{
		OnSandboxLoaded();
	}
}

FActiveSandboxFileStateViewModel::~FActiveSandboxFileStateViewModel()
{
	Model->OnLoadSandbox().RemoveAll(this);
	Model->OnLeaveSandbox().RemoveAll(this);
	Model->OnSandboxFilesChanged().RemoveAll(this);
}

void FActiveSandboxFileStateViewModel::RefreshItems()
{
	if (!Model->HasActiveSandbox() && AllUnfilteredItems.IsEmpty())
	{
		return;
	}
	
	if (!Model->HasActiveSandbox() && !AllUnfilteredItems.IsEmpty())
	{
		AllUnfilteredItems.Empty();
		NotifyItemsChanged();
		return;
	}
	
	bool bMadeChanges = false;
	TArray<TSharedPtr<FFileStateItem>> OldItems = MoveTemp(AllUnfilteredItems);
	const FString ActiveSandbox = Model->GetActiveSandboxPath();
	Model->EnumerateFileChanges(ActiveSandbox, 
		[this, &bMadeChanges, &OldItems](const FileSandboxCore::FSandboxedFileChangeInfo& InChange)
		{
			const FString& NonSandboxPath = InChange.Path;
			const FileSandboxCore::ESandboxFileChange Action = InChange.Action;
			const FDateTime Timestamp = InChange.Timestamp;
				
			const TSharedPtr<FFileStateItem>* ReusedItem = OldItems.FindByPredicate([&NonSandboxPath](const TSharedPtr<FFileStateItem>& Item)
			{
				return FPaths::IsSamePath(Item->NonSandboxFile, NonSandboxPath);
			});
			if (ReusedItem)
			{
				// Do this to make sure the item is up to date
				ReusedItem->Get()->Action = Action; 
				ReusedItem->Get()->Timestamp = Timestamp;
				
				AllUnfilteredItems.Emplace(*ReusedItem);
			}
			else
			{
				bMadeChanges = true;
				AllUnfilteredItems.Emplace(MakeShared<FFileStateItem>(NonSandboxPath, Action, Timestamp));
			}
				
			return FileSandboxCore::EBreakBehavior::Continue;
		});
	
	bMadeChanges |= OldItems.Num() != AllUnfilteredItems.Num();
	if (bMadeChanges)
	{
		NotifyItemsChanged();
	}
}

void FActiveSandboxFileStateViewModel::OnSandboxLoaded()
{
	RefreshItems();
}

void FActiveSandboxFileStateViewModel::OnSandboxLeft()
{
	RefreshItems();
}
}
