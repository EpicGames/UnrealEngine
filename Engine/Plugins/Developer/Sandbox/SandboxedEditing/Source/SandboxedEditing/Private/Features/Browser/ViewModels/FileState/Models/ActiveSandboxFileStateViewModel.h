// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticFileStateViewModel.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** Refreshes the file states of the active sandbox.  */
class FActiveSandboxFileStateViewModel : public FStaticFileStateViewModel
{
public:
	
	explicit FActiveSandboxFileStateViewModel(
		const TSharedRef<FSandboxSystemModel>& InModel,
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel,
		TMap<FName, TSharedRef<IFileStateColumnBehavior>> InColumns
		);
	~FActiveSandboxFileStateViewModel();
	
private:
	
	/** Used to get the file states from the active sandbox. */
	TSharedPtr<FSandboxSystemModel> Model;
	
	/** Refreshes the items from the active sandbox. */
	void RefreshItems();
	
	void OnSandboxLoaded();
	void OnSandboxLeft();
};
}

