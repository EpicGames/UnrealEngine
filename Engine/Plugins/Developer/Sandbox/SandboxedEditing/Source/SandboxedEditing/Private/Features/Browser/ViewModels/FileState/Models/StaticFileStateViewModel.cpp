// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticFileStateViewModel.h"

#include "Features/Browser/ViewModels/FileState/FilterFileStateViewModel.h"
#include "Features/Browser/ViewModels/FileState/IFileStateColumnBehavior.h"

namespace UE::SandboxedEditing
{
FStaticFileStateViewModel::FStaticFileStateViewModel(
	TMap<FName, TSharedRef<IFileStateColumnBehavior>> InColumns,
	TArray<TSharedPtr<FFileStateItem>> InItems,
	TSharedPtr<FFilterFileStateViewModel> InFilterViewModel
	)
	: Columns(MoveTemp(InColumns))
	, FilterViewModel(MoveTemp(InFilterViewModel))
	, AllUnfilteredItems(MoveTemp(InItems))
	, FilteredItems(AllUnfilteredItems)
{
	if (FilterViewModel)
	{
		FilterViewModel->OnFilterChanged().AddRaw(this, &FStaticFileStateViewModel::NotifyItemsChanged);
	}
}

FStaticFileStateViewModel::~FStaticFileStateViewModel()
{
	if (FilterViewModel)
	{
		FilterViewModel->OnFilterChanged().RemoveAll(this);
	}
}

TArray<FName> FStaticFileStateViewModel::GetDisplayedColumns() const
{
	TArray<FName> ColumnNames;
	Columns.GenerateKeyArray(ColumnNames);
	return ColumnNames;
}

void FStaticFileStateViewModel::NotifyItemsChanged()
{
	FilterItems();
	OnItemsChangedDelegate.Broadcast();
}

void FStaticFileStateViewModel::FilterItems()
{
	if (!FilterViewModel)
	{
		FilteredItems = AllUnfilteredItems;
		return;
	}

	FilteredItems.Empty(AllUnfilteredItems.Num());

	for (const TSharedPtr<FFileStateItem>& Item : AllUnfilteredItems)
	{
		if (ensure(Item) && FilterViewModel->PassesFilter(Item.ToSharedRef()))
		{
			FilteredItems.Add(Item);
		}
	}
}

EColumnSortMode::Type FStaticFileStateViewModel::GetColumnSortMode(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return PrimarySortMode;
	}
	if (ColumnId == SecondarySortedColumn)
	{
		return SecondarySortMode;
	}
	return EColumnSortMode::None;
}

EColumnSortPriority::Type FStaticFileStateViewModel::GetColumnSortPriority(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	if (ColumnId == SecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}
	return EColumnSortPriority::Max;
}

void FStaticFileStateViewModel::OnColumnSortModeChanged(
	const EColumnSortPriority::Type SortPriority,
	const FName& ColumnId,
	const EColumnSortMode::Type InSortMode)
{
	if (SortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedColumn = ColumnId;
		PrimarySortMode = InSortMode;

		// Cannot be primary and secondary at the same time
		if (ColumnId == SecondarySortedColumn)
		{
			SecondarySortedColumn = FName();
			SecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (SortPriority == EColumnSortPriority::Secondary)
	{
		SecondarySortedColumn = ColumnId;
		SecondarySortMode = InSortMode;

		// Cannot be primary and secondary at the same time
		if (ColumnId == PrimarySortedColumn)
		{
			PrimarySortedColumn = FName();
			PrimarySortMode = EColumnSortMode::None;
		}
	}

	RequestSort();
}

void FStaticFileStateViewModel::RequestSort()
{
	SortItems();
	OnItemsChangedDelegate.Broadcast();
}

void FStaticFileStateViewModel::SortItems()
{
	if (PrimarySortedColumn.IsNone())
	{
		return; // No sorting set
	}

	const TSharedRef<IFileStateColumnBehavior>* PrimaryColumnBehavior = Columns.Find(PrimarySortedColumn);
	if (!PrimaryColumnBehavior || !PrimaryColumnBehavior->Get().CanSort())
	{
		return;
	}

	const TSharedRef<IFileStateColumnBehavior>* SecondaryColumnBehavior = nullptr;
	if (!SecondarySortedColumn.IsNone())
	{
		SecondaryColumnBehavior = Columns.Find(SecondarySortedColumn);
		if (SecondaryColumnBehavior && !SecondaryColumnBehavior->Get().CanSort())
		{
			SecondaryColumnBehavior = nullptr;
		}
	}

	FilteredItems.Sort([&](const TSharedPtr<FFileStateItem>& Lhs, const TSharedPtr<FFileStateItem>& Rhs)
	{
		if (!Lhs || !Rhs)
		{
			return false;
		}

		// Try primary sort first
		if (PrimaryColumnBehavior->Get().SortBy(Lhs, Rhs, PrimarySortMode))
		{
			return true; // Lhs must be before Rhs based on primary sort
		}
		// Check if they're equal on primary column by testing reverse
		if (PrimaryColumnBehavior->Get().SortBy(Rhs, Lhs, PrimarySortMode))
		{
			return false; // Rhs must be before Lhs based on primary sort
		}

		// Equal on primary column - use secondary if available
		if (SecondaryColumnBehavior)
		{
			return SecondaryColumnBehavior->Get().SortBy(Lhs, Rhs, SecondarySortMode);
		}

		return false;
	});
}
}
