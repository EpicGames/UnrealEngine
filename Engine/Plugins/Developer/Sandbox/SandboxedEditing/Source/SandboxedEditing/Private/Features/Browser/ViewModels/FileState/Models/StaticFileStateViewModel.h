// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/IFileStateViewModel.h"

namespace UE::SandboxedEditing
{
class FFilterFileStateViewModel;
class IFileStateColumnBehavior;
class FSandboxListItem;

/**
 * Initialized with items at construction. 
 * 
 * The item array is not directly mutated by this class (TODO except for sorting columns). 
 * Subclasses may mutate the item array.
 * This can serve as a base implementation for more dynamic models.
 */
class FStaticFileStateViewModel : public IFileStateViewModel
{
public:
	
	explicit FStaticFileStateViewModel(
		TMap<FName, TSharedRef<IFileStateColumnBehavior>> InColumns,
		TArray<TSharedPtr<FFileStateItem>> InItems = {},
		TSharedPtr<FFilterFileStateViewModel> InFilterViewModel = nullptr
		);
	virtual ~FStaticFileStateViewModel() override;
	
	//~ Begin IFileStateViewModel Interface
	virtual const TArray<TSharedPtr<FFileStateItem>>& GetItems() const override { return FilteredItems; }
	virtual TArray<FName> GetDisplayedColumns() const override;
	virtual FSimpleMulticastDelegate& OnItemsChanged() override { return OnItemsChangedDelegate; }
	virtual EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const override;
	virtual EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const override;
	virtual void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode) override;
	//~ End IFileStateViewModel Interface

protected:
	
	/** All columns that can ever be displayed. */
	const TMap<FName, TSharedRef<IFileStateColumnBehavior>> Columns;
	
	/** Optional filter model. */
	const TSharedPtr<FFilterFileStateViewModel> FilterViewModel;
	
	/** The items to display. */
	TArray<TSharedPtr<FFileStateItem>> AllUnfilteredItems;
	
	/** Called by subclasses when they have changed AllUnfilteredItems. */
	void NotifyItemsChanged();
	
private:

	/** Invoked when items or sorting changes. */
	FSimpleMulticastDelegate OnItemsChangedDelegate;

	/** A filtered version of AllUnfilteredItems. */
	TArray<TSharedPtr<FFileStateItem>> FilteredItems;

	/** Sorting state */
	FName PrimarySortedColumn;
	EColumnSortMode::Type PrimarySortMode = EColumnSortMode::None;
	FName SecondarySortedColumn;
	EColumnSortMode::Type SecondarySortMode = EColumnSortMode::None;

	/** Refilters the items. */
	void FilterItems();

	/**
	 * Requests that the items be sorted according to the current sort column and mode.
	 * Refreshes the list after sorting.
	 */
	void RequestSort();

	/** Sorts the items according to the current sort column and mode. */
	void SortItems();
};
}

