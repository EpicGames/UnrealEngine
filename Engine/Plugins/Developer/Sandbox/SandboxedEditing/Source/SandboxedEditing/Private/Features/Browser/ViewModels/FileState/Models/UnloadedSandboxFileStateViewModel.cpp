// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnloadedSandboxFileStateViewModel.h"

#include "Features/Browser/ViewModels/FileState/FileStateItemUtils.h"
#include "Types/GatheredFileChanges.h"

namespace UE::SandboxedEditing
{
FUnloadedSandboxFileStateViewModel::FUnloadedSandboxFileStateViewModel(
	const TSharedRef<FSandboxSystemModel>& InModel,
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel,
	const TMap<FName, TSharedRef<IFileStateColumnBehavior>>& InColumns
	)
	: FStaticFileStateViewModel(InColumns, {}, InFilterViewModel)
	, Model(InModel)
{
	Model->OnLeaveSandbox().AddRaw(this, &FUnloadedSandboxFileStateViewModel::OnSandboxLeft);
}

FUnloadedSandboxFileStateViewModel::~FUnloadedSandboxFileStateViewModel()
{
	Model->OnLeaveSandbox().RemoveAll(this);
}

void FUnloadedSandboxFileStateViewModel::SetContent(const FString& InSandboxRoot)
{
	SelectedSandbox = InSandboxRoot;
	
	AllUnfilteredItems = MakeItemsFromFileChanges(
		Model->GatherFileChanges(InSandboxRoot)
		);
	NotifyItemsChanged();
}

void FUnloadedSandboxFileStateViewModel::ClearContent()
{
	SelectedSandbox.Reset();
	AllUnfilteredItems.Empty();
	NotifyItemsChanged();
}

void FUnloadedSandboxFileStateViewModel::OnSandboxLeft()
{
	if (SelectedSandbox)
	{
		SetContent(*SelectedSandbox);
	}
}
}
