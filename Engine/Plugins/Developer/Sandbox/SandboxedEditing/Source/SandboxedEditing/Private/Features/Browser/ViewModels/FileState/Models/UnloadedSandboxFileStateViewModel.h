// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticFileStateViewModel.h"
#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
/**
 * View model for displaying the file changes of an unloaded sandbox, i.e. when the engine is not sandboxed.
 * Used to display info about a sandbox in the browser.
 */
class FUnloadedSandboxFileStateViewModel : public FStaticFileStateViewModel
{
public:
	
	explicit FUnloadedSandboxFileStateViewModel(
		const TSharedRef<FSandboxSystemModel>& InModel,
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel,
		const TMap<FName, TSharedRef<IFileStateColumnBehavior>>& InColumns
		);
	~FUnloadedSandboxFileStateViewModel();
	
	/** Sets the content to be that of the specified sandbox. */
	void SetContent(const FString& InSandboxRoot);
	
	/** Clears the content. */
	void ClearContent();
	
private:
	
	/** The model through which we interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** The sandbox being viewed. */
	TOptional<FString> SelectedSandbox;
	
	/** Refreshes the content, in case the active sandbox changed files. */
	void OnSandboxLeft();
};
}

