// Copyright Epic Games, Inc. All Rights Reserved.

#include "AskUserAboutDirtyPackagesWorkflow.h"

#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FAskUserAboutDirtyPackagesWorkflow::FAskUserAboutDirtyPackagesWorkflow(
	const TSharedRef<FSandboxSystemModel>& InSandboxModel, FOnCleanupWorkflow InCleanupWorkflowDelegate
	)
	: SandboxModel(InSandboxModel)
	, CleanupWorkflowDelegate(MoveTemp(InCleanupWorkflowDelegate))
{}

void FAskUserAboutDirtyPackagesWorkflow::RevertDirtyPackages()
{
	const bool bSuccess = SandboxModel->RevertDirtyPackages();
	BroadcastEndWorkflowDelegates(bSuccess ? EWorkflowResult::Success : EWorkflowResult::Error);
}

void FAskUserAboutDirtyPackagesWorkflow::SaveDirtyPackages()
{
	const bool bSuccess = SandboxModel->SaveDirtyPackages();
	BroadcastEndWorkflowDelegates(bSuccess ? EWorkflowResult::Success : EWorkflowResult::Error);
}

void FAskUserAboutDirtyPackagesWorkflow::Cancel()
{
	BroadcastEndWorkflowDelegates(EWorkflowResult::Cancelled);
}

void FAskUserAboutDirtyPackagesWorkflow::BroadcastEndWorkflowDelegates(EWorkflowResult InResult)
{
	OnWorkflowEndedDelegate.Broadcast(InResult);
	CleanupWorkflowDelegate.ExecuteIfBound(InResult);
}
}
