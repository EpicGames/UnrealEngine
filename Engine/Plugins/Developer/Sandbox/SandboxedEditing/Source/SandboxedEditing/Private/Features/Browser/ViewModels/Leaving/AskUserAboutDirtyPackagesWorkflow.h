// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** Workflow for asking the user what to do about dirty packages, i.e. when leaving the sandbox. */
class FAskUserAboutDirtyPackagesWorkflow : public FNoncopyable
{
public:
	
	enum class EWorkflowResult { Success, Error, Cancelled };
	DECLARE_DELEGATE_OneParam(FOnCleanupWorkflow, EWorkflowResult)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndWorkflow, EWorkflowResult)
	
	explicit FAskUserAboutDirtyPackagesWorkflow(const TSharedRef<FSandboxSystemModel>& InSandboxModel, FOnCleanupWorkflow InCleanupWorkflowDelegate);
	
	/** Discards all dirty packages, and reloads them. */
	void RevertDirtyPackages();
	
	/** Saves all dirty packages. */
	void SaveDirtyPackages();
	
	/** Cancels this workflow */
	void Cancel();
	
	/** Invoked when this workflow ends. */
	FOnEndWorkflow& OnWorkflowEnded() { return OnWorkflowEndedDelegate; } 
	
private:
	
	/** The shared sandbox model. */
	const TSharedRef<FSandboxSystemModel> SandboxModel;
	
	/** Cleanup delegate invoked after OnWorkflowEndedDelegate. */
	const FOnCleanupWorkflow CleanupWorkflowDelegate;
	/** Invoked when this workflow ends. */
	FOnEndWorkflow OnWorkflowEndedDelegate;
	
	void BroadcastEndWorkflowDelegates(EWorkflowResult InResult);
};
}

