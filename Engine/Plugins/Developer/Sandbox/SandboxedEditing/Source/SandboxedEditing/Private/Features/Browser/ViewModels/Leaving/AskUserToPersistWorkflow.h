// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
enum class EAskUserToPersistResult : uint8
{
	/** The user wants to select files to persist. */
	ProceedToPersist,
	/** The users wants to leave without persisting. */
	LeaveWithoutPersist,
	/** User decided to cancel the operation. */
	Cancelled
};

/** Upon leaving the sandbox, asks the user whether they want to persist changes */
class FAskUserToPersistWorkflow : public FNoncopyable
{
public:
	
	DECLARE_DELEGATE_OneParam(FCleanupWorkflow, EAskUserToPersistResult);
	
	explicit FAskUserToPersistWorkflow(FCleanupWorkflow InCleanupWorkflow)
		: CleanupWorkflow(MoveTemp(InCleanupWorkflow))
	{}
	
	/** Proceed to present the dialogue for persisting files. Called by view. */
	void ProceedToPersist() { CleanupWorkflow.ExecuteIfBound(EAskUserToPersistResult::ProceedToPersist); }
	
	/** Leave the sandbox without persisting files. Called by view. */
	void LeaveWithoutPersist() { CleanupWorkflow.ExecuteIfBound(EAskUserToPersistResult::LeaveWithoutPersist); }
	
	/** The user decided to cancel the operation. Called by view. */
	void CancelLeave() { CleanupWorkflow.ExecuteIfBound(EAskUserToPersistResult::Cancelled); }
	
private:
	
	/** Invoked when the workflow ends. */
	FCleanupWorkflow CleanupWorkflow;
};
}

