// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeaveSandboxViewModel.h"

#include "Editor.h"
#include "Features/Browser/ViewModels/Persist/PersistWorkflowUserChoice.h"
#include "Features/Browser/ViewModels/Persist/PersistSandboxViewModel.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications.h"
#include "Types/GatheredFileChanges.h"

namespace UE::SandboxedEditing
{
FLeaveSandboxViewModel::FLeaveSandboxViewModel(
	const TSharedRef<FSandboxSystemModel> InModel, 
	const TSharedRef<FPersistSandboxViewModel> InPersistViewModel
	)
	: Model(InModel)
	, PersistViewModel(InPersistViewModel)
{}

FLeaveSandboxViewModel::~FLeaveSandboxViewModel()
{
	PersistViewModel->OnEndPersistWorkflow().RemoveAll(this);
}

void FLeaveSandboxViewModel::LeaveSandbox()
{
	if (GEditor && GEditor->IsPlaySessionInProgress())
	{
		ShowCannotLeaveDuringPlayMode();
		return;
	}

	if (!CanLeaveSandbox())
	{
		return;
	}
	
	if (Model->CanLeaveSandboxWithoutFurtherActions())
	{
		Model->LeaveSandbox();
		return;
	}
	
	const TArray<UPackage*> DirtyPackages = Model->GetInMemoryChanges();
	if (!DirtyPackages.IsEmpty())
	{
		LeaveWithDirtyPackagesWorkflow = MakeUnique<FAskUserAboutDirtyPackagesWorkflow>(
			Model, 
			FAskUserAboutDirtyPackagesWorkflow::FOnCleanupWorkflow::CreateRaw(this, &FLeaveSandboxViewModel::OnDirtyPackagesHandled)
			);
		OnCreateLeaveWithDirtyPackageWorkflowDelegate.Broadcast(*LeaveWithDirtyPackagesWorkflow);
		return;
	}
	
	const FileSandboxCore::FGatheredFileChanges FileChanges = Model->GatherActiveFileChanges();
	if (FileChanges.HasChanges())
	{
		AskUserToPersistWorkflow = MakeUnique<FAskUserToPersistWorkflow>(
			FAskUserToPersistWorkflow::FCleanupWorkflow::CreateRaw(this, &FLeaveSandboxViewModel::OnUserDecidedWhetherToPersist)
			);
		AskUserToPersistWorkflowDelegate.Broadcast(*AskUserToPersistWorkflow);
	}
}

bool FLeaveSandboxViewModel::CanLeaveSandbox(FText* OutReason) const
{
	return !LeaveWithDirtyPackagesWorkflow && !PersistViewModel->GetPersistWorkflow() && Model->IsAllowedToLeaveSandbox(OutReason);
}

void FLeaveSandboxViewModel::OnDirtyPackagesHandled(FAskUserAboutDirtyPackagesWorkflow::EWorkflowResult InResult)
{
	LeaveWithDirtyPackagesWorkflow.Reset();
	
	if (InResult == FAskUserAboutDirtyPackagesWorkflow::EWorkflowResult::Success)
	{
		// Proceed with the leave operation.
		LeaveSandbox();
	}
}

void FLeaveSandboxViewModel::OnUserDecidedWhetherToPersist(EAskUserToPersistResult InResult)
{
	AskUserToPersistWorkflow.Reset();

	switch (InResult)
	{
	case EAskUserToPersistResult::ProceedToPersist: 
		// Subscribe before starting the workflow because it can complete "immediately" (due to modal window).
		PersistViewModel->OnEndPersistWorkflow().AddRaw(this, &FLeaveSandboxViewModel::OnPersistOperationEnded);
		
		PersistViewModel->StartPersistWorkflow(Model->GatherActiveFileChanges());
		
		// If the workflow immediately ends (due to modal window) or fails, unsubscribe again
		if (!PersistViewModel->GetPersistWorkflow())
		{
			PersistViewModel->OnEndPersistWorkflow().RemoveAll(this);
		}
		break;
	case EAskUserToPersistResult::LeaveWithoutPersist: Model->LeaveSandbox(); break;
	case EAskUserToPersistResult::Cancelled: break;
	default: break;
	}
}

void FLeaveSandboxViewModel::OnPersistOperationEnded(EPersistWorkflowAction InAction) const
{
	if (InAction != EPersistWorkflowAction::Cancelled)
	{
		Model->LeaveSandbox();
	}
	
	PersistViewModel->OnEndPersistWorkflow().RemoveAll(this);
}
}
