// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AskUserAboutDirtyPackagesWorkflow.h"
#include "AskUserToPersistWorkflow.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Features/Browser/ViewModels/Persist/PersistSandboxViewModel.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
enum class EPersistWorkflowAction : uint8;

DECLARE_MULTICAST_DELEGATE_OneParam(FLeaveWithDirtyPackagesWorkflowDelegate, FAskUserAboutDirtyPackagesWorkflow&);
DECLARE_MULTICAST_DELEGATE_OneParam(FAskUserToPersistWorkflowDelegate, FAskUserToPersistWorkflow&);

/**
 * Handles the userflow of leaving a sandbox. 
 * 
 * All steps that may be performed are:
 * 1. Handle any in-memory changes to packages: either save all, or discard all.
 * 2. Ask the user to persist files changes, if any have been made.
 * 3. Persist files the user has selected.
 * 4. Leave the sandbox.
 */
class FLeaveSandboxViewModel : public FNoncopyable
{
public:
	
	explicit FLeaveSandboxViewModel(
		const TSharedRef<FSandboxSystemModel> InModel,
		const TSharedRef<FPersistSandboxViewModel> InPersistViewModel
		);
	~FLeaveSandboxViewModel();

	/** Leaves the active sandbox. */
	void LeaveSandbox();
	/** @return Whether the sandbox can be left */
	bool CanLeaveSandbox(FText* OutReason = nullptr) const; 
	
	/** @return Gets the current workflow */
	FAskUserAboutDirtyPackagesWorkflow* GetLeaveWithDirtyPackageWorkflow() const { return LeaveWithDirtyPackagesWorkflow.Get(); }
	/** @return Gets the current workflow */
	FAskUserToPersistWorkflow* GetAskUserToPersistWorkflow() const { return AskUserToPersistWorkflow.Get(); }
	
	/** Invoked when the user should be asked about what to do with dirty packages. */
	FLeaveWithDirtyPackagesWorkflowDelegate& OnCreateLeaveWithDirtyPackageWorkflow() { return OnCreateLeaveWithDirtyPackageWorkflowDelegate; }
	/** Invoked when the user should be asked whether they want to persist any files. */
	FAskUserToPersistWorkflowDelegate& OnAskUserToPersistWorkflow() { return AskUserToPersistWorkflowDelegate; }
	
private:
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;

	/** Used to go through the persist workflow. */
	const TSharedRef<FPersistSandboxViewModel> PersistViewModel;

	/** Handles dirty packages when leaving. */
	TUniquePtr<FAskUserAboutDirtyPackagesWorkflow> LeaveWithDirtyPackagesWorkflow;
	/** Asks the user whether they want to persist, leave without persisting, or cancel. */
	TUniquePtr<FAskUserToPersistWorkflow> AskUserToPersistWorkflow;
	
	/** Invoked when the user should be asked about what to do with dirty packages. */
	FLeaveWithDirtyPackagesWorkflowDelegate OnCreateLeaveWithDirtyPackageWorkflowDelegate;
	/** Invoked when the user should be asked whether they want to persist any files. */
	FAskUserToPersistWorkflowDelegate AskUserToPersistWorkflowDelegate;
	
	/** Invoked when LeaveWithDirtyPackagesWorkflow finishes. */
	void OnDirtyPackagesHandled(FAskUserAboutDirtyPackagesWorkflow::EWorkflowResult InResult);
	/** Invoked when the FAskUserToPersistWorkflow ends.*/
	void OnUserDecidedWhetherToPersist(EAskUserToPersistResult InResult);
	/** Invoked when the FPersistSandboxWorkflow has ended. */
	void OnPersistOperationEnded(EPersistWorkflowAction InAction) const;
};
}
