// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterSandboxViewModel.h"

#include "ISandboxColumnBehavior.h"

namespace UE::SandboxedEditing
{
FFilterSandboxViewModel::FFilterSandboxViewModel(TAttribute<TArray<TSharedRef<ISandboxColumnBehavior>>> InVisibleColumns)
	: VisibleColumns(MoveTemp(InVisibleColumns))
	, TextFilter(FSandboxTextFilter::FItemToStringArray::CreateRaw(this, &FFilterSandboxViewModel::ItemToString))
{
	TextFilter.OnChanged().AddLambda([this]
	{
		OnFilterChangedDelegate.Broadcast();
	});
}

FText FFilterSandboxViewModel::GetSearchText() const
{
	return TextFilter.GetRawFilterText();
}

void FFilterSandboxViewModel::SetSearchText(const FText& InSearchText)
{
	TextFilter.SetRawFilterText(InSearchText);
}

bool FFilterSandboxViewModel::PassesFilter(const TSharedRef<FSandboxListItem>& InItem) const
{
	return TextFilter.PassesFilter(InItem);
}

void FFilterSandboxViewModel::ItemToString(const TSharedRef<FSandboxListItem>& InItem, TArray<FString>& OutSearchTerms)
{
	for (const TSharedRef<ISandboxColumnBehavior>& Column : VisibleColumns.Get())
	{
		Column->PopulateSearchTerms(InItem, OutSearchTerms);
	}
}
}
