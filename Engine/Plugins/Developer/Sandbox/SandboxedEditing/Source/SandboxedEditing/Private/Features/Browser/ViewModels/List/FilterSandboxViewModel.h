// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SandboxListItem.h"
#include "Misc/Attribute.h"
#include "Misc/TextFilter.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class ISandboxColumnBehavior;
struct FSandboxInfo;

/** Knows of all filters for filtering sandbox items. */
class FFilterSandboxViewModel : public FNoncopyable
{
	using FSandboxTextFilter = TTextFilter<const TSharedRef<FSandboxListItem>&>;
public:
	
	explicit FFilterSandboxViewModel(TAttribute<TArray<TSharedRef<ISandboxColumnBehavior>>> InVisibleColumns);
	
	/** @return Current search text */
	FText GetSearchText() const;
	/** Sets the search text. Will trigger a refilter. */
	void SetSearchText(const FText& InSearchText);
	
	/** @return Whether the item can be displayed. */
	bool PassesFilter(const TSharedRef<FSandboxListItem>& InItem) const;
	
	/** @return Delegate invoked when any of the filters changes */
	FSimpleMulticastDelegate& OnFilterChanged() { return OnFilterChangedDelegate; }
	
private:
	
	/** Visible columns. Used for filtering visible columns by text. */
	const TAttribute<TArray<TSharedRef<ISandboxColumnBehavior>>> VisibleColumns;
	
	/** Does text filltering on items */
	FSandboxTextFilter TextFilter;
	
	/** Invoked when any of the filters changes */
	FSimpleMulticastDelegate OnFilterChangedDelegate;
	
	/** Converts an item to string. */
	void ItemToString(const TSharedRef<FSandboxListItem>& InItem, TArray<FString>& OutSearchTerms);
};
}

