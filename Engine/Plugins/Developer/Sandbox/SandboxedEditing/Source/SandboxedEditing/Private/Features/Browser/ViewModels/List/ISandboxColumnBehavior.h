// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/Columns/ISearchableColumnBehavior.h"
#include "Templates/SharedPointer.h"

namespace UE::SandboxedEditing
{
class FSandboxListItem;

/** Operations for columns that display sandboxes in the sandbox list view. */
class ISandboxColumnBehavior : public ISearchableColumnBehavior<TSharedPtr<FSandboxListItem>>
{};
}
