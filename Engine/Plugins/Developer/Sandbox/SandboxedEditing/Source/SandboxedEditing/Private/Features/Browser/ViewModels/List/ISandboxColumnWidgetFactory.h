// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
class FSandboxListItem;

/** Args for making a column widget in a row. */
struct FMakeSandboxColumnWidgetArgs
{
	/** The row data for this column widget */
	TSharedPtr<FSandboxListItem> RowData;
		
	/** Text that is being searched */
	TAttribute<FText> HighlightText;
	
	/** Whether the row is currently selected in the SListView. Required for some widgets, e.g. SInlineEditableTextBlock. */
	FIsSelected IsSelectedDelegate;

	explicit FMakeSandboxColumnWidgetArgs(TSharedPtr<FSandboxListItem> InRowData, TAttribute<FText> InHighlightText, FIsSelected InIsSelectedDelegate)
		: RowData(MoveTemp(InRowData))
		, HighlightText(MoveTemp(InHighlightText))
		, IsSelectedDelegate(MoveTemp(InIsSelectedDelegate))
	{}
};

/** Interface for creating Slate widgets for a column. */
class ISandboxColumnWidgetFactory
{
public:
	
	/** @return Info required to construct the column. */
	virtual SHeaderRow::FColumn::FArguments MakeColumnArguments() = 0;
	
	/** @return Widget that should be put into the row for this column */
	virtual TSharedRef<SWidget> MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs) = 0;

	virtual ~ISandboxColumnWidgetFactory() = default;
};
}
