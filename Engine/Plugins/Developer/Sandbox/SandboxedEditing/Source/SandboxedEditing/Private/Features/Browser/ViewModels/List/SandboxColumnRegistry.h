// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "SandboxColumns.h"
#include "Templates/SharedPointer.h"

namespace UE::SandboxedEditing
{
class ISandboxColumnBehavior;

struct FSandboxColumnRegistry
{
	/** The behavior of the columns for view models. */
	TMap<FName, TSharedRef<ISandboxColumnBehavior>> ColumnBehaviors;
	
	/** Creates the widgets for the columns. */
	TMap<FName, TSharedRef<ISandboxColumnWidgetFactory>> ColumnFactories;
	
	TArray<TSharedRef<ISandboxColumnBehavior>> GetBehaviorArray() const
	{
		TArray<TSharedRef<ISandboxColumnBehavior>> Columns;
		ColumnBehaviors.GenerateValueArray(Columns);
		return Columns;
	}
};
}