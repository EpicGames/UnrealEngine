// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::SandboxedEditing
{
class FSandboxListItem;
class ISandboxColumnBehavior;
class ISandboxColumnWidgetFactory;

/** Needed for SLATE_ARGUMENT (the macro is written in such a way that TMap is not parsed correctly). */
using FSandboxColumnFactoryMap = TMap<FName, TSharedRef<ISandboxColumnWidgetFactory>>;

constexpr FLazyName NameSandboxColumn(TEXT("NameSandboxColumn"));
constexpr FLazyName DescriptionSandboxColumn(TEXT("DescriptionSandboxColumn"));
constexpr FLazyName VersionSandboxNameColumn(TEXT("VersionSandboxNameColumn"));
constexpr FLazyName LastModifiedSandboxColumn(TEXT("LastModifiedSandboxColumn"));
}
