// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Models/SandboxInfo.h"

namespace UE::SandboxedEditing
{
/** A single item displayed in the SSandboxListView. */
class FSandboxListItem
{
public:
	
	/** Basic info about the sandbox */
	FSandboxInfo SandboxInfo;

	explicit FSandboxListItem(FSandboxInfo SandboxInfo)
		: SandboxInfo(MoveTemp(SandboxInfo))
	{}
};
}

