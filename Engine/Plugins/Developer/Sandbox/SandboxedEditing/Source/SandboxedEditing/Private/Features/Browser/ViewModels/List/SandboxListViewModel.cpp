// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxListViewModel.h"

#include "FilterSandboxViewModel.h"
#include "SandboxColumns.h"
#include "SandboxListItem.h"
#include "Templates/SharedPointer.h"

#define LOCTEXT_NAMESPACE "FSandboxListViewModel"

namespace UE::SandboxedEditing
{
FSandboxListViewModel::FSandboxListViewModel(
	const TSharedRef<FSandboxSystemModel>& InSandboxModel, const TSharedRef<FFilterSandboxViewModel>& InFilterViewModel,
	TMap<FName, TSharedRef<ISandboxColumnBehavior>> InColumns
	)
	: SandboxModel(InSandboxModel)
	, FilterViewModel(InFilterViewModel)
	, Columns(MoveTemp(InColumns))
{
	RefreshItems();
	
	SandboxModel->OnKnownSandboxesChanged().AddRaw(this, &FSandboxListViewModel::RefreshItems);
	FilterViewModel->OnFilterChanged().AddRaw(this, &FSandboxListViewModel::FilterItems);
}

FSandboxListViewModel::~FSandboxListViewModel()
{
	SandboxModel->OnKnownSandboxesChanged().RemoveAll(this);
	FilterViewModel->OnFilterChanged().RemoveAll(this);
}

TArray<FName> FSandboxListViewModel::GetDisplayedColumns() const
{
	TArray<FName> ColumnIds;
	for (const auto& Pair : Columns)
	{
		if (!HiddenColumns.Contains(Pair.Key))
		{
			ColumnIds.Add(Pair.Key);
		}
	}
	return ColumnIds;
}

bool FSandboxListViewModel::IsColumnVisible(FName InColumnName) const
{
	return !HiddenColumns.Contains(InColumnName);
}

void FSandboxListViewModel::SetColumnVisibility(FName InColumnName, bool bIsVisible)
{
	// Name column cannot be hidden
	if (InColumnName == NameSandboxColumn)
	{
		return;
	}

	const bool bIsCurrentlyVisible = !HiddenColumns.Contains(InColumnName);
	if (bIsVisible == bIsCurrentlyVisible)
	{
		return;
	}

	if (bIsVisible)
	{
		HiddenColumns.Remove(InColumnName);
	}
	else
	{
		HiddenColumns.Add(InColumnName);
	}

	OnItemsChangedDelegate.Broadcast();
}

EColumnSortPriority::Type FSandboxListViewModel::GetSortPriority(FName InColumnName) const
{
	// TODO DP:
	return EColumnSortPriority::None;
}

EColumnSortMode::Type FSandboxListViewModel::GetSortMode(FName InColumnName) const
{
	// TODO DP:
	return EColumnSortMode::None;
}

void FSandboxListViewModel::SetSortMode(const EColumnSortPriority::Type InSortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	// TODO DP:
}

void FSandboxListViewModel::RefreshItems()
{
	bool bMadeChanges = false;
	TArray<TSharedPtr<FSandboxListItem>> OldItems = MoveTemp(AllUnfilteredItems);
	SandboxModel->ForEachSandbox([this, &bMadeChanges, &OldItems](const FSandboxInfo& Info)
	{
		const TSharedPtr<FSandboxListItem>* ReusedItem = OldItems.FindByPredicate([&Info](const TSharedPtr<FSandboxListItem>& Item)
		{
			return FPaths::IsSamePath(Item->SandboxInfo.SandboxRoot, Info.SandboxRoot);
		});
		if (ReusedItem)
		{
			ReusedItem->Get()->SandboxInfo = Info; // Do this to make sure name and description are up to date.
			AllUnfilteredItems.Add(*ReusedItem);
		}
		else
		{
			bMadeChanges = true;
			AllUnfilteredItems.Emplace(MakeShared<FSandboxListItem>(Info));
		}
	});

	bMadeChanges |= OldItems.Num() != AllUnfilteredItems.Num();
	if (bMadeChanges)
	{
		FilterItems();
	}
}

void FSandboxListViewModel::FilterItems()
{
	FilteredItems.Empty(AllUnfilteredItems.Num());
	
	for (const TSharedPtr<FSandboxListItem>& Item : AllUnfilteredItems)
	{
		if (ensure(Item) && FilterViewModel->PassesFilter(Item.ToSharedRef()))
		{
			FilteredItems.Add(Item);
		}
	}
	
	SortItems();
	OnItemsChangedDelegate.Broadcast();
}

void FSandboxListViewModel::SortItems()
{
	// TODO DP: sort by sort mode & priority
	AllUnfilteredItems.Sort();
}
}

#undef LOCTEXT_NAMESPACE