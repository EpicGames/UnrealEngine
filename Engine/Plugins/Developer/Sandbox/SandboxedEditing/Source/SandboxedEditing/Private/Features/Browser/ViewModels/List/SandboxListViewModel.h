// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::SandboxedEditing
{
class FFilterSandboxViewModel;
class FSandboxListItem;
class ISandboxColumnBehavior;

/** Decides what is displayed in SSandboxListView. */
class FSandboxListViewModel : public FNoncopyable
{
public:
	
	explicit FSandboxListViewModel(
		const TSharedRef<FSandboxSystemModel>& InSandboxModel, const TSharedRef<FFilterSandboxViewModel>& InFilterViewModel,
		TMap<FName, TSharedRef<ISandboxColumnBehavior>> InColumns
		);
	~FSandboxListViewModel();
	
	/** @return The items to display */
	const TArray<TSharedPtr<FSandboxListItem>>& GetItems() const { return FilteredItems; }

	/** @return The columns that should be displayed (taking visibility into account). */
	TArray<FName> GetDisplayedColumns() const;

	/** @return Whether a column is visible. Name column is always visible. */
	bool IsColumnVisible(FName InColumnName) const;

	/** Sets whether a column is visible. Name column cannot be hidden. */
	void SetColumnVisibility(FName InColumnName, bool bIsVisible);
	
	/** @return The priority with which the column is sorted. */
	EColumnSortPriority::Type GetSortPriority(FName InColumnName) const;
	/** @return How the column is sorted */
	EColumnSortMode::Type GetSortMode(FName InColumnName) const;
	/** Changes the sort mode for the given column */
	void SetSortMode(const EColumnSortPriority::Type InSortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	
	/** Invoked when the displayed items changes (added / removed / sorted).*/
	FSimpleMulticastDelegate& OnItemsChanged() { return OnItemsChangedDelegate; }
	
private:
	
	/** The shared sandbox model. */
	const TSharedRef<FSandboxSystemModel> SandboxModel;
	/** Used to filter the items. */
	const TSharedRef<FFilterSandboxViewModel> FilterViewModel;
	
	/** All columns that can ever be displayed. */
	const TMap<FName, TSharedRef<ISandboxColumnBehavior>> Columns;

	/** Tracks which columns are visible. Name column is always visible. */
	TSet<FName> HiddenColumns;

	/** Contains all items. */
	TArray<TSharedPtr<FSandboxListItem>> AllUnfilteredItems;
	/** A filtered version of AllUnfilteredItems. */
	TArray<TSharedPtr<FSandboxListItem>> FilteredItems;
	
	/** Invoked when the displayed items changes (added / removed / sorted).*/
	FSimpleMulticastDelegate OnItemsChangedDelegate;
	
	/** Updates AllUnfilteredItems from the model. Tries to retain preexisting items. */
	void RefreshItems();
	
	/**  */
	void FilterItems();
	
	/** Sorts the items. */
	void SortItems();
};
}

