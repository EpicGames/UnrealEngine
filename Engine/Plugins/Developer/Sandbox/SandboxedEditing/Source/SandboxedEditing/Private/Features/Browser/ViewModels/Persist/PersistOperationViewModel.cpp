// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistOperationViewModel.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "PersistSandboxWorkflow.h"

namespace UE::SandboxedEditing
{
FPersistOperationViewModel::FPersistOperationViewModel(FPersistSandboxWorkflow& InWorkflow)
	: Workflow(InWorkflow)
	, FilesMarkedForPersist(true, Workflow.GetPersistableFiles().NonSandboxPaths.Num())
{}

void FPersistOperationViewModel::ConfirmPersist() const
{
	TArray<FString> FilesToPersist;
	const TArray<FString>& NonSandboxPaths = GetPersistableFiles().NonSandboxPaths;
	for (int32 Index = 0; Index < NonSandboxPaths.Num(); ++Index)
	{
		if (IsFileMarkedForPersist(Index))
		{
			FilesToPersist.Add(NonSandboxPaths[Index]);
		}
	}
	
	Workflow.ConfirmPersist(
		MoveTemp(FilesToPersist)
		);
}

void FPersistOperationViewModel::CancelPersist() const
{
	Workflow.CancelPersist();
}

void FPersistOperationViewModel::SetFilePersisted(int32 InIndex, bool bShouldPersist)
{
	FilesMarkedForPersist[InIndex] = bShouldPersist;
}

bool FPersistOperationViewModel::IsFileMarkedForPersist(int32 InIndex) const
{
	return FilesMarkedForPersist[InIndex];
}

void FPersistOperationViewModel::SetAllFilesPersisted(bool bAllFiles)
{
	FilesMarkedForPersist = TBitArray(bAllFiles, Workflow.GetPersistableFiles().NonSandboxPaths.Num());
}

bool FPersistOperationViewModel::AreAllFilesMarkedForPersist() const
{
	return Algo::AllOf(FilesMarkedForPersist);
}

bool FPersistOperationViewModel::AreAnyFilesMarkedForPersist() const
{
	return Algo::AnyOf(FilesMarkedForPersist);
}

const FileSandboxCore::FGatheredFileChanges& FPersistOperationViewModel::GetPersistableFiles() const
{
	return Workflow.GetPersistableFiles();
}
}
