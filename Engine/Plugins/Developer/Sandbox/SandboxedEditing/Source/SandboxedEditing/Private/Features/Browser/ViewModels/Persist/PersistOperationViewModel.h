// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "HAL/Platform.h"
#include "Types/GatheredFileChanges.h"

namespace UE::SandboxedEditing
{
class FPersistSandboxWorkflow;

/** View model for UI that lets the user click items they want to persist. */
class FPersistOperationViewModel : public FNoncopyable
{
public:
	
	explicit FPersistOperationViewModel(FPersistSandboxWorkflow& InWorkflow UE_LIFETIMEBOUND);
	
	/** Persists the files selected so far. Any follow-up operation, such as leaving the sandbox, will be performed. */
	void ConfirmPersist() const;
	
	/** Cancels the workflow. No follow-up operation, such as leaving the sandbox, will be performed. */
	void CancelPersist() const;
	
	/** 
	 * @param InIndex Index to GetPersistableFiles().NonSandboxPaths
	 * @param bShouldPersist Whether the files should be persisted
	 */
	void SetFilePersisted(int32 InIndex, bool bShouldPersist);
	/** 
	 * @param InIndex Index to GetPersistableFiles().NonSandboxPaths
	 * @return Whether the file is marked to be persisted 
	 */
	bool IsFileMarkedForPersist(int32 InIndex) const;
	
	/** Marks all files to be persisted. */
	void SetAllFilesPersisted(bool bAllFiles);
	
	/** @return Whether all files are marked to be persisted. */
	bool AreAllFilesMarkedForPersist() const;
	/** @return Whether any files are marked for persist. */
	bool AreAnyFilesMarkedForPersist() const;
	
	/** @return The modified files */
	const FileSandboxCore::FGatheredFileChanges& GetPersistableFiles() const;
	
private:
	
	/** The workflow to forward the persist and cancel events to. */
	FPersistSandboxWorkflow& Workflow;
	
	/** The files the user has selected to persist. Each index corresponds to PersistableFiles.NonSandboxPaths. */
	TBitArray<> FilesMarkedForPersist;
};
}

