// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistSandboxViewModel.h"

#include "PersistWorkflowUserChoice.h"
#include "Interface/Feedback/AggregatePersistFeedback.h"
#include "Persist/Feedback/SlowTaskPersistFeedback.h"
#include "Persist/Feedback/SummaryPersistFeedback.h"

namespace UE::SandboxedEditing
{
FPersistSandboxViewModel::FPersistSandboxViewModel(const TSharedRef<FSandboxSystemModel>& InModel)
	: Model(InModel)
{}

bool FPersistSandboxViewModel::StartPersistWorkflow(FileSandboxCore::FGatheredFileChanges InFileChanges, TConstArrayView<FString> InPreSelected)
{
	if (CanStartPersistWorkflow())
	{
		ActivePersistWorkflow = MakeUnique<FPersistSandboxWorkflow>(Model, MoveTemp(InFileChanges),
			FPersistSandboxWorkflow::FCleanupWorkflow::CreateRaw(this, &FPersistSandboxViewModel::OnCleanupPersistWorkflow)
			);
		OnStartPersistWorkflowDelegate.Broadcast(*ActivePersistWorkflow, InPreSelected);
		return true;
	}
	
	return false;
}

bool FPersistSandboxViewModel::StartPersistWorkflowForActiveSandbox(TConstArrayView<FString> InPreSelected)
{
	FileSandboxCore::FGatheredFileChanges FileChanges = Model->GatherActiveFileChanges();
	return FileChanges.HasChanges() && StartPersistWorkflow(MoveTemp(FileChanges), InPreSelected);
}

bool FPersistSandboxViewModel::CanStartPersistWorkflow() const
{
	return !ActivePersistWorkflow.IsValid();
}

bool FPersistSandboxViewModel::CanStartPersistWorkflowForActiveSandbox() const
{
	return CanStartPersistWorkflow() && Model->GatherActiveFileChanges().HasChanges();
}

void FPersistSandboxViewModel::OnCleanupPersistWorkflow(const FPersistWorkflowUserChoice& InPersistResult)
{
	if (InPersistResult.UserAction == EPersistWorkflowAction::Persist)
	{
		FileSandboxUI::FSlowTaskPersistFeedback ProgressBarFeedback(InPersistResult.FilesToPersist->Num());
		ProgressBarFeedback.SlowTask.MakeDialogDelayed(0.2f);
		FileSandboxUI::FSummaryPersistFeedback SummaryFeedback(ActivePersistWorkflow->GetPersistableFiles());
		FileSandboxCore::FAggregatePersistFeedback Aggregate({ &ProgressBarFeedback, &SummaryFeedback });
		Model->PersistFiles(*InPersistResult.FilesToPersist, &Aggregate);
		
		OnRequestPersistSummaryNotificationDelegate.Broadcast(SummaryFeedback.Summary);
	}
	
	ActivePersistWorkflow.Reset();
	OnEndPersistWorkflowDelegate.Broadcast(InPersistResult.UserAction);
}
}
