// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersistSandboxWorkflow.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxUI { struct FPersistSummary; }

namespace UE::SandboxedEditing
{
enum class EPersistWorkflowAction : uint8;
DECLARE_MULTICAST_DELEGATE_TwoParams(FPersistSandboxWorkflowDelegate, FPersistSandboxWorkflow&, TConstArrayView<FString> /*InPreSelectedFiles*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FPersistWorkflowActionDelegate, EPersistWorkflowAction);
DECLARE_MULTICAST_DELEGATE_OneParam(FRequestPersistSummaryNotificationDelegate, const FileSandboxUI::FPersistSummary&)

/** Handles the logic of persisting a sandbox. */
class FPersistSandboxViewModel : public FNoncopyable
{
public:
	
	explicit FPersistSandboxViewModel(const TSharedRef<FSandboxSystemModel>& InModel);

	/** Starts the persist workflow. */
	bool StartPersistWorkflow(FileSandboxCore::FGatheredFileChanges InFileChanges, TConstArrayView<FString> InPreSelected = {});
	/**
	 * Starts a persist workflow by using the changes of the active sandbox.
	 * @param 
	 */
	bool StartPersistWorkflowForActiveSandbox(TConstArrayView<FString> InPreSelected = {});
	
	/** @return Whether a persist workflow can be started */
	bool CanStartPersistWorkflow() const;
	/** @return Whether a persist workflow can be started for the active sandbox */
	bool CanStartPersistWorkflowForActiveSandbox() const;
	
	/** @return The current persist workflow */
	FPersistSandboxWorkflow* GetPersistWorkflow() const { return ActivePersistWorkflow.Get(); }
	
	/** Invoked when the persist workflow is started. */
	FPersistSandboxWorkflowDelegate& OnStartPersistWorkflow() { return OnStartPersistWorkflowDelegate; }
	/** Invoked when the persist workflow has ended. */
	FPersistWorkflowActionDelegate& OnEndPersistWorkflow() { return OnEndPersistWorkflowDelegate; }
	
	/** Invoked when a persist operation is performed to show a notification for the result. */
	FRequestPersistSummaryNotificationDelegate& OnRequestPersistSummaryNotification() { return OnRequestPersistSummaryNotificationDelegate; }
	
private:
	
	/** Used to persist changes via the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** The active persist workflow. */
	TUniquePtr<FPersistSandboxWorkflow> ActivePersistWorkflow;
	
	/** Invoked when the persist workflow is started. */
	FPersistSandboxWorkflowDelegate OnStartPersistWorkflowDelegate;
	/** Invoked when the persist workflow has ended. */
	FPersistWorkflowActionDelegate OnEndPersistWorkflowDelegate;
	
	/** Invoked when a persist operation is performed to show a notification for the result. */
	FRequestPersistSummaryNotificationDelegate OnRequestPersistSummaryNotificationDelegate;
	
	/** Handles the workflow ending, possibly persisting InFilesToPersist. */
	void OnCleanupPersistWorkflow(const FPersistWorkflowUserChoice& InPersistResult);
};
}

