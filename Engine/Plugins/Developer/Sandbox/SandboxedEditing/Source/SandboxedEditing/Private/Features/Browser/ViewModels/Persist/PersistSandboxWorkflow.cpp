// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistSandboxWorkflow.h"

#include "PersistWorkflowUserChoice.h"
#include "Algo/Transform.h"
#include "Types/GatheredFileChanges.h"

namespace UE::SandboxedEditing
{
FPersistSandboxWorkflow::FPersistSandboxWorkflow(
	const TSharedRef<FSandboxSystemModel>& InModel, FileSandboxCore::FGatheredFileChanges InChanges,
	FCleanupWorkflow InOnWorkflowEnded
	)
	: Model(InModel)
	, OnCleanupWorkflowDelegate(MoveTemp(InOnWorkflowEnded))
	, PersistableFiles(InChanges)
{}

void FPersistSandboxWorkflow::ConfirmPersist(TArray<FString> InFilesToPersist) const
{
	EndWorkflow(InFilesToPersist.IsEmpty() 
		? FPersistWorkflowUserChoice::MakePersistNone()
		: FPersistWorkflowUserChoice::MakePersist(MoveTemp(InFilesToPersist))
		);
}

void FPersistSandboxWorkflow::CancelPersist() const
{
	EndWorkflow(
		FPersistWorkflowUserChoice::MakeCancelled()
		);
}

void FPersistSandboxWorkflow::EndWorkflow(const FPersistWorkflowUserChoice& InResult) const
{
	OnWorkflowEndedDelegate.Broadcast();
	OnCleanupWorkflowDelegate.ExecuteIfBound(InResult);
}
}
