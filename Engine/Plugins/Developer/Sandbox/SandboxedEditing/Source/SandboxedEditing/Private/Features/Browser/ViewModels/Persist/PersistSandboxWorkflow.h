// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/GatheredFileChanges.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
struct FPersistWorkflowUserChoice;

enum class EPersistWorkflowResult : uint8
{
	/** Files were persisted, or the user decided to confirm without any files to persist. */
	Persisted,
	/** User decided to cancel the operation. Do not continue with follow-up operations, such as leaving the sandbox. */
	Cancelled
};

/**
 * Represents a temporary application state in which user operation is choosing files to persist. 
 * Effectively, this is a view-model representation of the pop-up dialogue, in which files to persist are chosen.
 */
class FPersistSandboxWorkflow : public FNoncopyable
{
public:
	
	/** @param FilesToPersist The files the user selected to persist. Unset if the operation is cancelled. Can be set but empty. */
	DECLARE_DELEGATE_OneParam(FCleanupWorkflow, const FPersistWorkflowUserChoice&);
	
	explicit FPersistSandboxWorkflow(
		const TSharedRef<FSandboxSystemModel>& InModel, FileSandboxCore::FGatheredFileChanges InChanges,
		FCleanupWorkflow InOnWorkflowEnded
		);
	
	/** Persists the files selected so far. Any follow-up operation, such as leaving the sandbox, will be performed. */
	void ConfirmPersist(TArray<FString> InFilesToPersist) const;
	
	/** Cancels the workflow. No follow-up operation, such as leaving the sandbox, will be performed. */
	void CancelPersist() const;
	
	/** @return The modified files */
	const FileSandboxCore::FGatheredFileChanges& GetPersistableFiles() const { return PersistableFiles; }
	
	/** Invoked when the workflow ends. */
	FSimpleMulticastDelegate& OnWorkflowEnded() { return OnWorkflowEndedDelegate; }

private:
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** Invoked when the workflow ends. */
	FSimpleMulticastDelegate OnWorkflowEndedDelegate;
	/** Invoked when the workflow ends. */
	const FCleanupWorkflow OnCleanupWorkflowDelegate;
	
	/** The files that can be persisted. */
	const FileSandboxCore::FGatheredFileChanges PersistableFiles;
	
	void EndWorkflow(const FPersistWorkflowUserChoice& InResult) const;
};
}
