// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"

namespace UE::SandboxedEditing
{
/** Action to take when FPersistSandboxWorkflow completes. */
enum class EPersistWorkflowAction : uint8
{
	/** Files were selected for persist. Any follow-up operation, such as leaving, should be performed. */
	Persist,
	
	/** User explicitly choose not to persist any of the files. Any follow-up operation, such as leaving, should be performed. */
	PersistNone,
	
	/** User decided to cancel the operation. Do not continue with follow-up operations, such as leaving the sandbox. */
	Cancelled
};

/** Describes the user decision for FPersistSandboxWorkflow. */
struct FPersistWorkflowUserChoice
{
	/** Determines what action the user chose to take. */
	EPersistWorkflowAction UserAction;
	
	/** Only set if UserAction == EPersistWorkflowAction::Persist. */
	TOptional<TArray<FString>> FilesToPersist;
	
	static FPersistWorkflowUserChoice MakePersist(TArray<FString> InFileToPersist)
	{
		return FPersistWorkflowUserChoice(EPersistWorkflowAction::Persist, MoveTemp(InFileToPersist));
	}
	static FPersistWorkflowUserChoice MakePersistNone() { return FPersistWorkflowUserChoice(EPersistWorkflowAction::PersistNone); }
	static FPersistWorkflowUserChoice MakeCancelled() { return FPersistWorkflowUserChoice(EPersistWorkflowAction::Cancelled); }
};
}
