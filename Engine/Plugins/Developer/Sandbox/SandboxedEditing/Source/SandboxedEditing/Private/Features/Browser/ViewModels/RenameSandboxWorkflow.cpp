// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenameSandboxWorkflow.h"

#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FRenameSandboxWorkflow::FRenameSandboxWorkflow(
	FString InRenamedRoot, 
	const TSharedRef<FSandboxSystemModel>& InModel,
	FSimpleDelegate InOnWorkflowEndedDelegate
	)
	: Model(InModel)
	, RenamedSandboxRoot(MoveTemp(InRenamedRoot))
	, InitialName(Model->GetSandboxInfo(RenamedSandboxRoot)->Name)
	, OnWorkflowEndedDelegate(MoveTemp(InOnWorkflowEndedDelegate))
{}

void FRenameSandboxWorkflow::Confirm()
{
	if (IsNameValid() && Model->RenameSandbox(RenamedSandboxRoot, NewName))
	{
		OnSandboxRenamedDelegate.Broadcast(InitialName, NewName);
		OnWorkflowEndedDelegate.ExecuteIfBound();
	}
}

void FRenameSandboxWorkflow::Cancel()
{
	OnWorkflowEndedDelegate.ExecuteIfBound();
}

bool FRenameSandboxWorkflow::IsNameValid(FText* OutErrorText) const
{
	if (!bHasChangedName)
	{
		return true;
	}
	
	return Model->CanRenameSandbox(RenamedSandboxRoot, NewName, OutErrorText);
}
}
