// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSandboxRenamed, const FString& /*InOldName*/, const FString& /*InNewName*/);

class FSandboxSystemModel;

/** Workflow for renaming a sandbox. */
class FRenameSandboxWorkflow : public FNoncopyable
{
public:
	
	explicit FRenameSandboxWorkflow(
		FString InRenamedRoot,
		const TSharedRef<FSandboxSystemModel>& InModel, FSimpleDelegate InOnWorkflowEndedDelegate
		);
	
	/** Renames the sandbox if the name is valid, and stops the operation if successful. */
	void Confirm();
	/** Cancels the operation. */
	void Cancel();
	
	/** Sets the that the sandbox should be renamed to. */
	void SetName(FString InName)
	{
		bHasChangedName |= InitialName != InName;
		NewName = InName;
	}
	
	/** @return Whether the current name is valid, and an optional error message if this function returns false. */
	bool IsNameValid(FText* OutErrorText = nullptr) const;
	
	/** @return The root directory of the sandbox being renamed */
	const FString& GetRenamedSandboxRoot() const { return RenamedSandboxRoot; }
	
	/** Invoked after the sandbox has been successfully renamed. */
	FOnSandboxRenamed& OnSandboxRenamed() { return OnSandboxRenamedDelegate; }

private:  
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** The root directory of the sandbox being renamed */
	const FString RenamedSandboxRoot;
	/** The name the sandbox had at the beginning. */
	const FString InitialName;
	
	/** Invoked after the sandbox has been successfully renamed. */
	FOnSandboxRenamed OnSandboxRenamedDelegate;
	/** Invoked when the workflow ends. */
	const FSimpleDelegate OnWorkflowEndedDelegate;
	
	/** The name to give the sandbox. */
	FString NewName;
	
	bool bHasChangedName = false;
};
}

