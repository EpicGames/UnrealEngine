// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxControlsViewModel.h"

#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications.h"

namespace UE::SandboxedEditing
{
FSandboxControlsViewModel::FSandboxControlsViewModel(const TSharedRef<FSandboxSystemModel>& InModel)
	: Model(InModel)
{}

void FSandboxControlsViewModel::CancelWorkflows()
{
	if (CurrentCreationWorkflow)
	{
		CurrentCreationWorkflow->Cancel();
	}
	
	if (CurrentRenameWorkflow)
	{
		CurrentRenameWorkflow->Cancel();
	}
}

void FSandboxControlsViewModel::StartCreationWorkflow()
{
	if (CanStartCreationWorkflow())
	{
		CurrentCreationWorkflow = MakeUnique<FSandboxCreationWorkflow>(
			Model,
			FSimpleDelegate::CreateRaw(this, &FSandboxControlsViewModel::HandleCreationWorkflowEnded)
			);
		OnCreationWorkflowStartedDelegate.Broadcast(*CurrentCreationWorkflow);
	}
}

bool FSandboxControlsViewModel::CanStartCreationWorkflow() const
{
	return !HasActiveWorkflow();
}

bool FSandboxControlsViewModel::CanLoadSandbox(const FString& InSandboxRoot, FText* OutReason) const
{
	return Model->CanLoadSandbox(InSandboxRoot);
}

void FSandboxControlsViewModel::LoadSandbox(const FString& InSandboxRoot)
{
	if (!CanLoadSandbox(InSandboxRoot))
	{
		return;
	}

	if (InMemoryChangesNotification.IsValid())
	{
		// We are actively waiting for a user to make a decision on entry into the sandbox. 
		return;
	}
	
	// Check for in-memory changes before loading
	const TArray<UPackage*> DirtyPackages = Model->GetInMemoryChanges();
	if (!DirtyPackages.IsEmpty())
	{
		if (CurrentCreationWorkflow)
		{
			CurrentCreationWorkflow->Cancel();
		}

		// Get sandbox name for display
		const TOptional<FSandboxInfo> SandboxInfo = Model->GetSandboxInfo(InSandboxRoot);
		const FString SandboxName = SandboxInfo.IsSet() ? SandboxInfo->Name : TEXT("Sandbox");

		TWeakPtr<FSandboxSystemModel> WeakModel = Model; 
		// Show warning notification with Discard/Cancel options
		InMemoryChangesNotification = ShowInMemoryChangesWarning(
			SandboxName,
			// OnDiscard: Discard changes and load sandbox
			[WeakModel, InSandboxRoot]()
			{
				if (TSharedPtr<FSandboxSystemModel> Model = WeakModel.Pin())
				{
					Model->RevertDirtyPackages();
					Model->LoadSandbox(InSandboxRoot);
				}
			},
			// OnCancel: Do nothing
			[]()
			{
				// User cancelled, do nothing
			}
		);
		return;
	}

	LoadSandboxInternal(InSandboxRoot);
}

void FSandboxControlsViewModel::LoadSandboxInternal(const FString& InSandboxRoot)
{
	if (CurrentCreationWorkflow)
	{
		CurrentCreationWorkflow->Cancel();
	}

	Model->LoadSandbox(InSandboxRoot);
}

bool FSandboxControlsViewModel::CanDeleteSandbox(const FString& InSandboxRoot, FText* OutReason) const
{
	return Model->CanDeleteSandbox(InSandboxRoot, OutReason);
}

void FSandboxControlsViewModel::DeleteSandbox(const FString& InSandboxRoot)
{
	CancelWorkflows();
	Model->DeleteSandbox(InSandboxRoot);
}

bool FSandboxControlsViewModel::CanRenameSandbox(const FString& InSandboxRoot, FText* OutError) const
{
	return !HasActiveWorkflow() && Model->IsAllowedToRenameSandbox(InSandboxRoot, OutError);
}

void FSandboxControlsViewModel::StartRenameWorkflow(const FString& InSandboxRoot)
{
	if (CanRenameSandbox(InSandboxRoot))
	{
		CurrentRenameWorkflow = MakeUnique<FRenameSandboxWorkflow>(
			InSandboxRoot, Model, 
			FSimpleDelegate::CreateRaw(this, &FSandboxControlsViewModel::HandleRenameWorkflowEnded)
			);
		OnRenameWorkflowStartedDelegate.Broadcast(*CurrentRenameWorkflow);
	}
}

bool FSandboxControlsViewModel::CanStartExportWorkflow() const
{
	return !HasActiveWorkflow();
}

void FSandboxControlsViewModel::StartExportWorkflow(const TArray<FString>& InSandboxRoots)
{
	if (CanStartExportWorkflow() && !InSandboxRoots.IsEmpty())
	{
		CurrentExportWorkflow = MakeUnique<FExportWorkflow>(
			InSandboxRoots,
			FSimpleDelegate::CreateRaw(this, &FSandboxControlsViewModel::HandleExportWorkflowEnded)
			);
		OnExportWorkflowStartedDelegate.Broadcast(*CurrentExportWorkflow);
	}
}

bool FSandboxControlsViewModel::CanStartImportWorkflow() const
{
	return !HasActiveWorkflow();
}

void FSandboxControlsViewModel::StartImportWorkflow()
{
	if (CanStartImportWorkflow())
	{
		CurrentImportWorkflow = MakeUnique<FImportWorkflow>(
			FSimpleDelegate::CreateRaw(this, &FSandboxControlsViewModel::HandleImportWorkflowEnded)
			);
		OnImportWorkflowStartedDelegate.Broadcast(*CurrentImportWorkflow);
	}
}

void FSandboxControlsViewModel::HandleCreationWorkflowEnded()
{
	CurrentCreationWorkflow.Reset();
	OnCreationWorkflowEndedDelegate.Broadcast();
}

void FSandboxControlsViewModel::HandleRenameWorkflowEnded()
{
	CurrentRenameWorkflow.Reset();
	OnRenameWorkflowEndedDelegate.Broadcast();
}

void FSandboxControlsViewModel::HandleExportWorkflowEnded()
{
	CurrentExportWorkflow.Reset();
	OnExportWorkflowEndedDelegate.Broadcast();
}

void FSandboxControlsViewModel::HandleImportWorkflowEnded()
{
	CurrentImportWorkflow.Reset();
	OnImportWorkflowEndedDelegate.Broadcast();
}
}
