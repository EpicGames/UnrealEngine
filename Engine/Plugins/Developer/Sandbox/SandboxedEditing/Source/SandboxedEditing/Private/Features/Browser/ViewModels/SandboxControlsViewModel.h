// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "RenameSandboxWorkflow.h"
#include "SandboxCreationWorkflow.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Transfer/ExportWorkflow.h"
#include "Transfer/ImportWorkflow.h"

class SNotificationItem;

namespace UE::SandboxedEditing
{
DECLARE_MULTICAST_DELEGATE_OneParam(FCreationWorkflowDelegate, FSandboxCreationWorkflow&);
DECLARE_MULTICAST_DELEGATE_OneParam(FRenameWorkflowDelegate, FRenameSandboxWorkflow&);
DECLARE_MULTICAST_DELEGATE_OneParam(FExportWorkflowDelegate, FExportWorkflow&);
DECLARE_MULTICAST_DELEGATE_OneParam(FImportWorkflowDelegate, FImportWorkflow&);

/**
 * Handles actions that can be performed with FSandboxListItems, such as deleting, renaming, loading.
 *
 * These actions are primarily present on the controls widget in the form of buttons.
 * Some of them can be performed in the list view, such as deleting or renaming.
 */
class FSandboxControlsViewModel : public FNoncopyable
{
public:
	
	explicit FSandboxControlsViewModel(const TSharedRef<FSandboxSystemModel>& InModel);
	
	/** Cancels any workflows that may be active. */
	void CancelWorkflows();
	
	/** Starts a workflow for creating a new sandbox. */
	void StartCreationWorkflow();
	/** @return Whether a new sandbox creation workflow can currently be started. */
	bool CanStartCreationWorkflow() const;
	/** @return The currently active creation workflow. */
	FSandboxCreationWorkflow* GetCurrentCreationWorkflow() const { return CurrentCreationWorkflow.Get(); }

	/** @return Whether the sandbox can be loaded. */
	bool CanLoadSandbox(const FString& InSandboxRoot, FText* OutReason = nullptr) const;
	/** Loads the sandbox. */
	void LoadSandbox(const FString& InSandboxRoot);

	/** @return Whether the sandbox can be deleted. */
	bool CanDeleteSandbox(const FString& InSandboxRoot, FText* OutReason = nullptr) const;
	/** Deletes the sandbox. */
	void DeleteSandbox(const FString& InSandboxRoot);
	
	/** @return Whether a rename operation can be started for the sandbox. */
	bool CanRenameSandbox(const FString& InSandboxRoot, FText* OutError = nullptr) const;
	/** Starts a rename operation for the sandbox. */
	void StartRenameWorkflow(const FString& InSandboxRoot);
	/** @return The currently active rename workflow. */
	FRenameSandboxWorkflow* GetCurrentRenameWorkflow() const { return CurrentRenameWorkflow.Get(); }
	
	/** @return Whether an export operation can be performed. */
	bool CanStartExportWorkflow() const;
	/** Starts an export workflow */
	void StartExportWorkflow(const TArray<FString>& InSandboxRoots);
	/** @return The currently active export workflow. */
	FExportWorkflow* GetCurrentExportWorkflow() const { return CurrentExportWorkflow.Get(); }
	
	/** @return Whether an import operation can be performed. */
	bool CanStartImportWorkflow() const;
	/** Starts an import workflow. */
	void StartImportWorkflow();
	/** @return The currently active import workflow. */
	FImportWorkflow* GetCurrentImportWorkflow() const { return CurrentImportWorkflow.Get(); }
	
	/** @return Delegate that is invoked when the creation workflow starts. */
	FCreationWorkflowDelegate& OnCreationWorkflowStarted() { return OnCreationWorkflowStartedDelegate; }
	/** @return Delegate that is invoked when the creation workflow ends. */
	FSimpleMulticastDelegate& OnCreationWorkflowEnded() { return OnCreationWorkflowEndedDelegate; }
	
	/** @return Delegate that is invoked when the rename workflow starts. */
	FRenameWorkflowDelegate& OnRenameWorkflowStarted() { return OnRenameWorkflowStartedDelegate; }
	/** @return Delegate that is invoked when the rename workflow ends. */
	FSimpleMulticastDelegate& OnRenameWorkflowEnded() { return OnRenameWorkflowEndedDelegate; }
	
	/** @return Delegate that is invoked when the export workflow starts. */
	FExportWorkflowDelegate& OnExportWorkflowStarted() { return OnExportWorkflowStartedDelegate; }
	/** @return Delegate that is invoked when the export workflow ends. */
	FSimpleMulticastDelegate& OnExportWorkflowEnded() { return OnExportWorkflowEndedDelegate; }
	
	/** @return Delegate that is invoked when the import workflow starts. */
	FImportWorkflowDelegate& OnImportWorkflowStarted() { return OnImportWorkflowStartedDelegate; }
	/** @return Delegate that is invoked when the import workflow ends. */
	FSimpleMulticastDelegate& OnImportWorkflowEnded() { return OnImportWorkflowEndedDelegate; }

private:

	/** Loads the sandbox without checking for in-memory changes. */
	void LoadSandboxInternal(const FString& InSandboxRoot);

	/** The underlying model that performs the mutations. */
	const TSharedRef<FSandboxSystemModel> Model;

	/** Active notification for in-memory changes warning. Lifetime is not bound to this class and only used to make sure we do not have multiple notification to the user. */
	TWeakPtr<SNotificationItem> InMemoryChangesNotification;
	
	/** Set if the user is currently in the process of creating a sandbox. */
	TUniquePtr<FSandboxCreationWorkflow> CurrentCreationWorkflow;
	/** Set if the user is currently in the process of renaming a sandbox. */
	TUniquePtr<FRenameSandboxWorkflow> CurrentRenameWorkflow;
	/** Set if the user is currently in the process of exporting sandboxes. */
	TUniquePtr<FExportWorkflow> CurrentExportWorkflow;
	/** Set if the user is currently in the process of importing sandboxes. */
	TUniquePtr<FImportWorkflow> CurrentImportWorkflow;
	
	/** Invoked when the creation workflow starts. */
	FCreationWorkflowDelegate OnCreationWorkflowStartedDelegate;
	/** Invoked when the creation workflow ends. */
	FSimpleMulticastDelegate OnCreationWorkflowEndedDelegate;
	
	/** Invoked when the rename workflow starts. */
	FRenameWorkflowDelegate OnRenameWorkflowStartedDelegate;
	/** Invoked when the rename workflow ends. */
	FSimpleMulticastDelegate OnRenameWorkflowEndedDelegate;
	
	/** Invoked when the export workflow starts. */
	FExportWorkflowDelegate OnExportWorkflowStartedDelegate;
	/** Invoked when the export workflow ends. */
	FSimpleMulticastDelegate OnExportWorkflowEndedDelegate;
	
	/** Invoked when the import workflow starts. */
	FImportWorkflowDelegate OnImportWorkflowStartedDelegate;
	/** Invoked when the import workflow ends. */
	FSimpleMulticastDelegate OnImportWorkflowEndedDelegate;

	/** Called by CurrentCreationWorkflow when the workflow ends. */
	void HandleCreationWorkflowEnded();
	/** Called by CurrentRenameWorkflow when the workflow ends. */
	void HandleRenameWorkflowEnded();
	/** Called by CurrentExportWorkflow when the workflow ends. */
	void HandleExportWorkflowEnded();
	/** Called by CurrentImportWorkflow when the workflow ends. */
	void HandleImportWorkflowEnded();
	
	/** @return Whether a workflow is active */
	bool HasActiveWorkflow() const { return CurrentCreationWorkflow || CurrentRenameWorkflow || CurrentExportWorkflow || CurrentImportWorkflow; }
};
}

