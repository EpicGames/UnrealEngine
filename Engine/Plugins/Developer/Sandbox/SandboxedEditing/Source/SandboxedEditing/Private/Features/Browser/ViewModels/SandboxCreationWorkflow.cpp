// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxCreationWorkflow.h"

#include "Framework/Models/CreateSandboxArgs.h"
#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FSandboxCreationWorkflow::FSandboxCreationWorkflow(const TSharedRef<FSandboxSystemModel>& InModel, FSimpleDelegate InOnWorkflowEndedDelegate)
	: Model(InModel)
	, OnWorkflowEndedDelegate(MoveTemp(InOnWorkflowEndedDelegate))
{}

void FSandboxCreationWorkflow::Confirm()
{
	if (!CanConfirm())
	{
		return;
	}
	
	if (!Model->CreateNewSandbox(FCreateSandboxArgs(Name, Description)))
	{
		return;
	}
	
	OnWorkflowEndedDelegate.ExecuteIfBound();
}

void FSandboxCreationWorkflow::Cancel()
{
	OnWorkflowEndedDelegate.ExecuteIfBound();
}

bool FSandboxCreationWorkflow::CanConfirm(FText* OutReason) const
{
	return IsValidName(Name, OutReason) && Model->CanCreateNewSandbox(Name, OutReason);
}

void FSandboxCreationWorkflow::SetName(const FString& InNewName)
{
	Name = InNewName;
}

bool FSandboxCreationWorkflow::IsValidName(const FString& InName, FText* OutReason) const
{
	return Model->CanCreateNewSandbox(InName, OutReason);
}

void FSandboxCreationWorkflow::SetDescription(const FString& InNewDescription)
{
	Description = InNewDescription;
}
}
