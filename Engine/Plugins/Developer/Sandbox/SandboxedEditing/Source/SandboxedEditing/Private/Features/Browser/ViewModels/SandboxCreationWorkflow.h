// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** 
 * View-model managing the workflow for creating a new sandbox.
 * 
 * The user can type the name, edit other data, etc. 
 * At any point they can confirm or cancel the workflow. 
 */
class FSandboxCreationWorkflow : public FNoncopyable
{
public:
	
	explicit FSandboxCreationWorkflow(const TSharedRef<FSandboxSystemModel>& InModel, FSimpleDelegate InOnWorkflowEndedDelegate);
	
	/** Finishes the workflow, if possible, resulting in the creation of the sandbox. */
	void Confirm();
	/** Stops the workflow. */
	void Cancel();
	
	/** @return Whether the workflow can be confirmed, i.e. whether the sandbox can be created. */
	bool CanConfirm(FText* OutReason = nullptr) const;
	
	/** Sets the name the user wants to give the sandbox. The name needn't be valid, however, it will prevent CanConfirm from returning true. */
	void SetName(const FString& InNewName);
	/** @return Whether this name is a valid name to set. */
	bool IsValidName(const FString& InName, FText* OutReason = nullptr) const;
	
	/** Sets the description. There are no restrictions on the description. */
	void SetDescription(const FString& InNewDescription);
	
private:  
	
	/** The model used to interact with the sandbox system. */
	const TSharedRef<FSandboxSystemModel> Model;
	
	/** Invoked when the workflow ends. */
	const FSimpleDelegate OnWorkflowEndedDelegate;
	
	/** The name the user wants to give. */
	FString Name;
	/** The description the user wants to give. */
	FString Description;
};
}

