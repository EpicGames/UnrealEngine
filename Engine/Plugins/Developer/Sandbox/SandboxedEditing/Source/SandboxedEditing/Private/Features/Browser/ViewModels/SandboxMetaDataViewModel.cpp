// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxMetaDataViewModel.h"

#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"

namespace UE::SandboxedEditing
{
FSandboxMetaDataViewModel::FSandboxMetaDataViewModel(const TSharedRef<FSandboxSystemModel>& InModel)
	: Model(InModel)
{}

TOptional<FString> FSandboxMetaDataViewModel::GetDescription(const FString& InSandboxRoot) const
{
	const TOptional<FSandboxInfo> Info = Model->GetSandboxInfo(InSandboxRoot);
	return Info ? Info->Description : TOptional<FString>();
}

void FSandboxMetaDataViewModel::SetDescription(const FString& InSandboxRoot, const FString& InNewDescription)
{
	Model->SetDescription(InSandboxRoot, InNewDescription);
}
}
