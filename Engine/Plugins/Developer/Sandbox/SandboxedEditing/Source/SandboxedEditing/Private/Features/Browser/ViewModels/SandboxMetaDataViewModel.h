// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** Handles editing metadata about sandboxes, such as name and description. */
class FSandboxMetaDataViewModel : public FNoncopyable
{
public:
	
	explicit FSandboxMetaDataViewModel(const TSharedRef<FSandboxSystemModel>& InModel);
	
	/** @return The description of the given sandbox. Unset if the sandbox root does not exist. */
	TOptional<FString> GetDescription(const FString& InSandboxRoot) const;
	
	/** Sets the description of the sandbox */
	void SetDescription(const FString& InSandboxRoot, const FString& InNewDescription);
	
private:
	
	/** The model used to query the sandboxes. */
	const TSharedRef<FSandboxSystemModel> Model;
};
}

