// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportWorkflow.h"

#include "ExportWorkflowResult.h"
#include "GlobalSandboxedEditingEditorSettings.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopedSlowTask.h"
#include "Utils/ZipTransferUtils.h"

#define LOCTEXT_NAMESPACE "FExportWorkflow"

namespace UE::SandboxedEditing
{
namespace ExportDetail
{
static bool IsDirectoryPath(const FString& InPath)
{
	return FPathViews::GetExtension(InPath).Len() == 0;
}

static void ExportSingle(const FString& InExportedSandboxPath, const FString& InExportPath, FExportWorkflowResult& OutResult)
{
	const TOptional<FString> Name = SandboxModel::GetSandboxName(InExportedSandboxPath);
	if (!ensure(Name))
	{
		return;
	}
	
	const bool bIsDirectory = IsDirectoryPath(InExportPath);
	const FString FinalExportPath = bIsDirectory ? FPaths::Combine(InExportPath, *Name) : InExportPath;
	
	// Zip compression can take some time, so start a slow task.
	FScopedSlowTask SlowTask(
		1.f, 
		Name ? FText::Format(LOCTEXT("ZipSingle.ItemFmt", "Exporting {0}"), FText::AsCultureInvariant(Name.Get(TEXT("")))) : FText::GetEmpty()
		);
	SlowTask.MakeDialogDelayed(0.5f);
	
	const bool bHasError = !FileSandboxCore::ZipUtils::ExportSandboxToZip(InExportedSandboxPath, FinalExportPath);
	OutResult.ExportResult = bHasError ? EExportWorkflowResult::SomeErrors : EExportWorkflowResult::Success;
	if (bHasError)
	{
		OutResult.SandboxesWithErrors.Add(InExportedSandboxPath);
	}
}

static void ExportMulti(const TArray<FString>& InExportedSandboxPaths, const FString& InExportBasePath, FExportWorkflowResult& OutResult)
{
	bool bAnyErrors = false;
	const auto AddError = [&OutResult, &bAnyErrors](FString InSandbox)
	{
		bAnyErrors = true;
		OutResult.SandboxesWithErrors.Emplace(MoveTemp(InSandbox));
	};
	
	// Zip compression can take some time, so start a slow task.
	FScopedSlowTask SlowTask(InExportedSandboxPaths.Num());
	SlowTask.MakeDialogDelayed(0.5f);
	
	for (const FString& SandboxRootPath : InExportedSandboxPaths)
	{
		const TOptional<FString> Name = SandboxModel::GetSandboxName(SandboxRootPath);
		SlowTask.EnterProgressFrame(
			1.f, 
			Name ? FText::Format(LOCTEXT("ZipMulti.ItemFmt", "Exporting {0}"), FText::AsCultureInvariant(*Name)) : FText::GetEmpty()
			);
		
		if (!ensure(Name))
		{
			AddError(SandboxRootPath);
			continue;
		}
		
		const FString Output = FPaths::Combine(InExportBasePath, *Name);
		if (!FileSandboxCore::ZipUtils::ExportSandboxToZip(SandboxRootPath, Output))
		{
			AddError(SandboxRootPath);
		}
	}
		
	OutResult.ExportResult = bAnyErrors ? EExportWorkflowResult::SomeErrors : EExportWorkflowResult::Success;
}
}

FExportWorkflow::FExportWorkflow(TArray<FString> InSandboxPaths, FSimpleDelegate InOnWorkflowEndedDelegate)
	: ExportedSandboxPaths(MoveTemp(InSandboxPaths))
	, OnWorkflowEndedDelegate(MoveTemp(InOnWorkflowEndedDelegate))
{
	check(!ExportedSandboxPaths.IsEmpty());
}

bool FExportWorkflow::CanExport()
{
	return CanSingleExport() || CanMultiExport();
}

void FExportWorkflow::TryExportOrCancel()
{
	if (!CanExport())
	{
		Cancel();
		return;
	}
	
	FExportWorkflowResult Result { .ExportResult = EExportWorkflowResult::SomeErrors };
	
	if (CanSingleExport())
	{
		ExportDetail::ExportSingle(ExportedSandboxPaths[0], ExportPath, Result);
	}
	else if (CanMultiExport())
	{
		ExportDetail::ExportMulti(ExportedSandboxPaths, ExportPath, Result);
	}
	
	ExportEndedDelegate.Broadcast(Result);
	OnWorkflowEndedDelegate.Execute();
}

void FExportWorkflow::Cancel()
{
	ExportEndedDelegate.Broadcast(FExportWorkflowResult{ .ExportResult = EExportWorkflowResult::Cancelled });
	OnWorkflowEndedDelegate.Execute();
}

TArray<FString> FExportWorkflow::GetFilesOverridenByExportDirectory(const FString& InNewExportRoot) const
{
	check(GetExportType() == EExportPathType::Directory);
	TArray<FString> Result;
	
	for (const FString& SandboxRootPath : ExportedSandboxPaths)
	{
		const TOptional<FString> Name = SandboxModel::GetSandboxName(SandboxRootPath);
		if (!Name)
		{
			continue;
		}
		
		const FString BaseFile = FPaths::Combine(InNewExportRoot, *Name);
		const FString WithExtension = FPaths::SetExtension(BaseFile, TEXT("zip"));
		const bool bExists = FPaths::FileExists(WithExtension) || FPaths::DirectoryExists(WithExtension);
		if (bExists)
		{
			Result.Add(WithExtension);
		}
	}
	
	return Result;
}

void FExportWorkflow::SetExportPath(FString InNewExportRoot)
{
	// For zip export, the system dialogue can specify a path or a full filename. But when exporting multiple items, only a directory is valid.
	if (GetExportType() == EExportPathType::Directory 
		&& !ensure(ExportDetail::IsDirectoryPath(InNewExportRoot)))
	{
		return;
	}
	
	ExportPath = MoveTemp(InNewExportRoot);
	UGlobalSandboxedEditingEditorSettings::Get()->SetDefaultExportDirectory(ExportPath);
}

const FString& FExportWorkflow::GetExportPath() const
{
	return ExportPath;
}

EExportPathType FExportWorkflow::GetExportType() const
{
	return ExportedSandboxPaths.Num() == 1 
		? EExportPathType::Zip 
		: EExportPathType::Directory;
}

FString FExportWorkflow::GetDefaultExportDirectory() const
{
	UGlobalSandboxedEditingEditorSettings* Settings = UGlobalSandboxedEditingEditorSettings::Get();
	const FString& DefaultExport = Settings->GetDefaultExportDirectory();
	// If the user has never exported, they may want to export to where they last imported from.
	return DefaultExport.IsEmpty() ? Settings->GetDefaultImportDirectory() : DefaultExport;
}

FString FExportWorkflow::GetDefaultSandboxFileName() const
{
	if (GetExportType() == EExportPathType::Directory)
	{
		return TEXT("");
	}
	
	if (const TOptional<FString> SandboxName = SandboxModel::GetSandboxName(ExportedSandboxPaths[0]))
	{
		return FPaths::SetExtension(
			FPaths::Combine(GetDefaultExportDirectory(), *SandboxName),
			TEXT("zip")
			);
	}
	
	return TEXT("");
}

bool FExportWorkflow::CanSingleExport() const
{
	return ExportedSandboxPaths.Num() == 1;
}

bool FExportWorkflow::CanMultiExport() const
{
	const bool bIsDirectory = ExportDetail::IsDirectoryPath(ExportPath);
	return ExportedSandboxPaths.Num() > 1 && ensure(bIsDirectory);
}
}

#undef LOCTEXT_NAMESPACE
