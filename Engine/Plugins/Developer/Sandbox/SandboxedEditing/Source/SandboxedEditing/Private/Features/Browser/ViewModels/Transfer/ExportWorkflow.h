// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
struct FExportWorkflowResult;

DECLARE_MULTICAST_DELEGATE_OneParam(FSandboxExportPathDelegate, const FString& InDefaultName);
DECLARE_MULTICAST_DELEGATE_OneParam(FExportEndedDelegate, const FExportWorkflowResult& InExportResult);

/** What type of path the user should select. */
enum class EExportPathType : uint8
{
	/** Select a directory to export multiple sandboxes to. */
	Directory,
	/** Select a file name for a single sandbox to be exported. */
	Zip,
	
	Count
};

/** Workflow for exporting sandboxes from Unreal. */
class FExportWorkflow : public FNoncopyable
{
public:
	
	explicit FExportWorkflow(TArray<FString> InSandboxPaths, FSimpleDelegate InOnWorkflowEndedDelegate);

	/** @return Whether the sandbox selection can be exported. */
	bool CanExport();
	/** Exports the sandbox selection. */
	void TryExportOrCancel();
	
	/** Cancels the operation. */
	void Cancel();
	
	/** @return List of files that would be overriden if this path is chosen. */
	TArray<FString> GetFilesOverridenByExportDirectory(const FString& InNewExportRoot) const;
	
	/** Sets the export path. */
	void SetExportPath(FString InNewExportRoot);
	/** @return The export path */
	const FString& GetExportPath() const;
	
	/** @return What type of path the user should select. */
	EExportPathType GetExportType() const;
	
	/** @return Default path to suggest to the user. */
	FString GetDefaultExportDirectory() const;
	/** @return Default filename to suggest to the user. */
	FString GetDefaultSandboxFileName() const;
	
	/** @return The sandboxes that are being exported. */
	TConstArrayView<FString> GetExportedSandboxRootPaths() const { return ExportedSandboxPaths; }
	
	/** Invoked when the operation ends. */
	FExportEndedDelegate& OnExportEnded() { return ExportEndedDelegate; }
	
private:
	
	/** The sandboxes that are being exported. */
	const TArray<FString> ExportedSandboxPaths;
	
	/** Invoked when the workflow ends. */
	const FSimpleDelegate OnWorkflowEndedDelegate;
	
	/** Invoked when the operation ends. */
	FExportEndedDelegate ExportEndedDelegate;
	
	/**
	 * This has a different meaning depending on ExportedSandboxPaths.Num:
	 * - Num() == 1: This is a file path to a .zip to export to, or a directory path (in which case the repository name is used as suggested name).
	 * - Num() > 1: This is a directory path to place the .zips in.
	 */
	FString ExportPath;
	
	bool CanSingleExport() const;
	bool CanMultiExport() const;
};
}
