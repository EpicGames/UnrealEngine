// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
/** Gives an indication of the workflow's result. */
enum class EExportWorkflowResult : uint8
{
	Success,
	SomeErrors,
	Cancelled,
	
	Count
};

/** Describes the result of a finished export workflow. */
struct FExportWorkflowResult
{
	/** Gives an indication of the workflow's result. */
	EExportWorkflowResult ExportResult;
	
	/** Sandboxes that failed to be exported. */
	TArray<FString> SandboxesWithErrors;
};
}
