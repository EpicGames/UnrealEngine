// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

namespace UE::SandboxedEditing
{
/** Result of validating user selected files before import. */
struct FImportValidationResult
{
	/**
	 * The root paths of sandboxes that already exist and would be overwritten by the import operation. 
	 * Ask the user to confirm overwrite. 
	 */
	TArray<FString> AskForOverwrite;
	
	/** Paths to zips that the user selected but that do not contain any valid sandboxes. */
	TArray<FString> InvalidZips;
};
}