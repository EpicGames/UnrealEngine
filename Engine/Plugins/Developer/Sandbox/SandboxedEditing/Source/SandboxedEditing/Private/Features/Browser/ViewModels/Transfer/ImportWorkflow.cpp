// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportWorkflow.h"

#include "Algo/Transform.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "GlobalSandboxedEditingEditorSettings.h"
#include "HAL/PlatformFileManager.h"
#include "ImportWorkflowResult.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Utils/ZipTransferUtils.h"

#define LOCTEXT_NAMESPACE "FImportWorkflow"

namespace UE::SandboxedEditing
{
namespace ImportWorkflowDetail
{
static bool IsDirectoryEmpty(const FString& DirectoryPath)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	bool bHasFiles = false;
	PlatformFile.IterateDirectory(
		*DirectoryPath,
		[&bHasFiles](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			bHasFiles = true;
			return false; 
		});

	return !bHasFiles;
}

static FString GetImportToPath(const FString& InSandboxName)
{
	return FPaths::Combine(SandboxModel::GetBaseSandboxDirectory(), InSandboxName);
}

static void ValidateFilesToImport(
	const TArray<FString>& InFilesToImport,
	const TArray<FileSandboxCore::ZipUtils::FImportInspectionResult>& InFileInspections,
	FImportValidationResult& OutResult
	)
{
	for (int32 Index = 0; Index < InFilesToImport.Num(); ++Index)
	{
		const FString& FileToImport = InFilesToImport[Index];
		const FileSandboxCore::ZipUtils::FImportInspectionResult& Inspection = InFileInspections[Index];
		
		if (!Inspection.IsValid())
		{
			OutResult.InvalidZips.Add(FileToImport);
			continue;
		}
		
		const FString OutputPath = GetImportToPath(Inspection.SandboxName);
		if (!IsDirectoryEmpty(OutputPath))
		{
			OutResult.AskForOverwrite.Add(OutputPath);
		}
	}
}
} 

FImportWorkflow::FImportWorkflow(FSimpleDelegate InOnWorkflowEndedDelegate)
	: OnWorkflowEndedDelegate(MoveTemp(InOnWorkflowEndedDelegate))
{}

void FImportWorkflow::TryImportOrCancel()
{
	if (FilesToImport.IsEmpty())
	{
		Cancel();
		return;
	}
	
	// Zip compression can take some time, so start a slow task.
	FScopedSlowTask SlowTask(FilesToImport.Num(), LOCTEXT("Import.Title", "Importing sandbox"));
	SlowTask.MakeDialogDelayed(0.5f);
	
	FImportWorkflowResult Result { .ResultCode = EImportWorkflowResult::Success };
	for (int32 Index = 0; Index < FilesToImport.Num(); ++Index)
	{
		const FString& FileToImport = FilesToImport[Index];
		const FileSandboxCore::ZipUtils::FImportInspectionResult& Inspection = FileInspections[Index];
		if (!Inspection.IsValid())
		{
			SlowTask.EnterProgressFrame(1.f);
			continue;
		}
		
		const FString& SandboxName = Inspection.SandboxName;
		SlowTask.EnterProgressFrame(1.f, FText::Format(LOCTEXT("Import.ItemFmt", "Importing {0}"), FText::AsCultureInvariant(SandboxName)));
		
		const FString ImportToPath = ImportWorkflowDetail::GetImportToPath(SandboxName);
		const bool bSuccess = FileSandboxCore::ZipUtils::ImportSandboxFromZip(Inspection, ImportToPath);
		if (!bSuccess)
		{
			Result.ResultCode = EImportWorkflowResult::SomeErrors;
			Result.Errors.Add(FileToImport);
		}
	}
	
	ImportEndedDelegate.Broadcast(Result);
	OnWorkflowEndedDelegate.Execute();
}

void FImportWorkflow::Cancel()
{
	ImportEndedDelegate.Broadcast(FImportWorkflowResult{ .ResultCode = EImportWorkflowResult::Cancelled });
	OnWorkflowEndedDelegate.Execute();
}

FImportValidationResult FImportWorkflow::SetFilesToImport(TArray<FString> InFiles)
{
	using namespace FileSandboxCore::ZipUtils;
	if (InFiles.IsEmpty())
	{
		return FImportValidationResult();
	}
	
	// Most likely all files are in the same directory. If not, too bad, can only have one.
	UGlobalSandboxedEditingEditorSettings::Get()->SetDefaultImportDirectory(InFiles[0]);
	
	FImportValidationResult Result;
	
	FilesToImport = MoveTemp(InFiles);
	FileInspections.Empty(FilesToImport.Num());
	for (int32 Index = 0; Index < FilesToImport.Num(); ++Index)
	{
		const FString& File = FilesToImport[Index];
		
		if (TOptional<FImportInspectionResult> InspectionResult = InspectFileForImport(File))
		{
			FileInspections.Emplace(MoveTemp(*InspectionResult));
		}
		else
		{
			// If we're unable to read the zip, then just mark it as invalid and don't add it to the list.
			Result.InvalidZips.Add(File);
			
			FilesToImport.RemoveAt(Index);
			--Index;
		}
	}
	
	ImportWorkflowDetail::ValidateFilesToImport(FilesToImport, FileInspections, Result);
	return Result;
}

FString FImportWorkflow::GetDefaultImportDirectory() const
{
	UGlobalSandboxedEditingEditorSettings* Settings = UGlobalSandboxedEditingEditorSettings::Get();
	const FString& DefaultImport = Settings->GetDefaultImportDirectory();
	// If the user has never imported, they may want to import from where they last exported to.
	return DefaultImport.IsEmpty() ? Settings->GetDefaultExportDirectory() : DefaultImport;
}
}

#undef LOCTEXT_NAMESPACE