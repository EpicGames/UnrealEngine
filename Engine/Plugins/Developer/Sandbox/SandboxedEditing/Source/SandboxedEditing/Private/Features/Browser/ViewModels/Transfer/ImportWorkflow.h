// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "FileUtilities/ZipArchiveReader.h"
#include "ImportValidationResult.h"
#include "Templates/UnrealTemplate.h"
#include "Utils/ZipTransferUtils.h"

namespace UE::FileSandboxCore::ZipUtils
{
struct FImportInspectionResult;
}

namespace UE::SandboxedEditing
{
struct FImportWorkflowResult;
DECLARE_MULTICAST_DELEGATE_OneParam(FImportEndedDelegate, const FImportWorkflowResult& InImportResult);

/** Workflow for importing sandboxes into Unreal. */
class FImportWorkflow : public FNoncopyable
{
public:
	
	explicit FImportWorkflow(FSimpleDelegate InOnWorkflowEndedDelegate);
	
	void TryImportOrCancel();
	
	void Cancel();
	
	/** Sets the files that are supposed */
	FImportValidationResult SetFilesToImport(TArray<FString> InFiles);
	/** @return The files that will be imported. */
	const TArray<FString>& GetFilesToImport() const { return FilesToImport; }
	
	/** @return Default path to suggest to the user. */
	FString GetDefaultImportDirectory() const;
	
	/** Invoked when the operation ends. */
	FImportEndedDelegate& OnImportEnded() { return ImportEndedDelegate; }
	
private:
	
	/** Invoked when the workflow ends. */
	const FSimpleDelegate OnWorkflowEndedDelegate;
	
	/** Paths to the files the user has selected for import. */
	TArray<FString> FilesToImport;
	/** Validation result for each selected file. */
	TArray<FileSandboxCore::ZipUtils::FImportInspectionResult> FileInspections;
	
	/** Invoked when the operation ends. */
	FImportEndedDelegate ImportEndedDelegate;
};
}
