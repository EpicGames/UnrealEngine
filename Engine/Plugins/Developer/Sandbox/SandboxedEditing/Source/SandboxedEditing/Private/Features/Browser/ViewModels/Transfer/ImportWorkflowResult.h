// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

namespace UE::SandboxedEditing
{
/** Gives an indication of the workflow's result. */
enum class EImportWorkflowResult : uint8
{
	Success,
	SomeErrors,
	Cancelled,
	
	Count
};

/** Describes the result of the entire import operation (multiple sandboxes can be imported at the same time). */
struct FImportWorkflowResult
{
	/** Gives an indication of the workflow's result. */
	EImportWorkflowResult ResultCode;
	
	/** The errors that occured. If empty, the operation was successful. */
	TArray<FString> Errors;
};
}