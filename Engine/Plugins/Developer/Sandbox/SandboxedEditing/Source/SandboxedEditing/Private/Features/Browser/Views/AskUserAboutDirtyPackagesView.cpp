// Copyright Epic Games, Inc. All Rights Reserved.

#include "AskUserAboutDirtyPackagesView.h"

#include "Features/Browser/ViewModels/Leaving/LeaveSandboxViewModel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FNotificationView"

namespace UE::SandboxedEditing
{
FAskUserAboutDirtyPackagesView::FAskUserAboutDirtyPackagesView(const TSharedRef<FLeaveSandboxViewModel>& InLeaveViewModel)
	: LeaveViewModel(InLeaveViewModel)
{
	LeaveViewModel->OnCreateLeaveWithDirtyPackageWorkflow().AddRaw(this, &FAskUserAboutDirtyPackagesView::ShowNotification);
}

FAskUserAboutDirtyPackagesView::~FAskUserAboutDirtyPackagesView()
{
	LeaveViewModel->OnCreateLeaveWithDirtyPackageWorkflow().RemoveAll(this);
	
	if (FAskUserAboutDirtyPackagesWorkflow* Workflow = LeaveViewModel->GetLeaveWithDirtyPackageWorkflow())
	{
		Workflow->OnWorkflowEnded().Remove(HandleWorkflowEndHandle);
	}
}

void FAskUserAboutDirtyPackagesView::ShowNotification(FAskUserAboutDirtyPackagesWorkflow& InWorkflow)
{
	HideNotification();
	HandleWorkflowEndHandle = InWorkflow.OnWorkflowEnded().AddLambda([this](auto){ HideNotification(); });
	
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	
	FNotificationInfo NotificationInfo(LOCTEXT("Unsaved.Title", "In-memory changes"));
	NotificationInfo.SubText = LOCTEXT("Unsaved.SubText", "You must either save or discard unsaved assets before leaving.");
	NotificationInfo.bFireAndForget = false;
	NotificationInfo.ButtonDetails = 
	{
		FNotificationButtonInfo(
			LOCTEXT("Unsaved.SaveAll.Title", "Save all"),
			LOCTEXT("Unsaved.SaveAll.ToolTip", "Save all unsaved assets"),
			FSimpleDelegate::CreateRaw(this, &FAskUserAboutDirtyPackagesView::HandleSaveAll),
			SNotificationItem::CS_None
			),
		FNotificationButtonInfo(
			LOCTEXT("Unsaved.DiscardAll.Title", "Discard"),
			LOCTEXT("Unsaved.DiscardAll.ToolTip", "Discard all unsaved assets"),
			FSimpleDelegate::CreateRaw(this, &FAskUserAboutDirtyPackagesView::HandleRevert),
			SNotificationItem::CS_None
			),
		FNotificationButtonInfo(
			LOCTEXT("Unsaved.Cancel.Title", "Cancel"),
			LOCTEXT("Unsaved.Cancel.ToolTip", "Does not leave the sandbox"),
			FSimpleDelegate::CreateRaw(this, &FAskUserAboutDirtyPackagesView::HandleCancel),
			SNotificationItem::CS_None
			)
	};
	Notification = NotificationManager.AddNotification(NotificationInfo);
}

void FAskUserAboutDirtyPackagesView::HandleSaveAll() const
{
	if (FAskUserAboutDirtyPackagesWorkflow* Workflow = LeaveViewModel->GetLeaveWithDirtyPackageWorkflow())
	{
		Workflow->SaveDirtyPackages();
	}
}

void FAskUserAboutDirtyPackagesView::HandleRevert() const
{
	if (FAskUserAboutDirtyPackagesWorkflow* Workflow = LeaveViewModel->GetLeaveWithDirtyPackageWorkflow())
	{
		Workflow->RevertDirtyPackages();
	}
}

void FAskUserAboutDirtyPackagesView::HandleCancel() const
{
	if (FAskUserAboutDirtyPackagesWorkflow* Workflow = LeaveViewModel->GetLeaveWithDirtyPackageWorkflow())
	{
		Workflow->Cancel();
	}
}

void FAskUserAboutDirtyPackagesView::HideNotification()
{
	if (Notification)
	{
		Notification->ExpireAndFadeout();
		Notification.Reset();
	}
}
}

#undef LOCTEXT_NAMESPACE