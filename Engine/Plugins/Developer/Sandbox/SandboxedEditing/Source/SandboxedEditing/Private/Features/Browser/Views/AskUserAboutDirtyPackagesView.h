// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class SNotificationItem;

namespace UE::SandboxedEditing
{
class FLeaveSandboxViewModel;
class FAskUserAboutDirtyPackagesWorkflow;

/** Visualizes the notification for leaving the sandbox with dirty packages. */
class FAskUserAboutDirtyPackagesView : public FNoncopyable
{
public:
	
	explicit FAskUserAboutDirtyPackagesView(const TSharedRef<FLeaveSandboxViewModel>& InLeaveViewModel);
	~FAskUserAboutDirtyPackagesView();
	
private:
	
	/** Notifies us when a FLeaveSandboxViewModel starts. */
	const TSharedRef<FLeaveSandboxViewModel> LeaveViewModel;

	/** Whether currently showing the notification for when the user is trying to leave the sandbox even though there are modified assets. */
	TSharedPtr<SNotificationItem> Notification;
	
	FDelegateHandle HandleWorkflowEndHandle;

	/** Show the notification for when the user is trying to leave the sandbox even though there are modified assets. */
	void ShowNotification(FAskUserAboutDirtyPackagesWorkflow& InWorkflow);
	
	/** Saves all dirty assets, hides the notification, and proceeds with the leave operation. */
	void HandleSaveAll() const;
	/** Discards all dirty assets, hides the notification, and proceeds with the leave operation. */
	void HandleRevert() const;
	/** Cancels the workflow. */
	void HandleCancel() const;
	
	/** Hides the notification. */
	void HideNotification();
};
}

