// Copyright Epic Games, Inc. All Rights Reserved.

#include "AskUserToPersistView.h"

#include "SandboxedEditingSettings.h"
#include "Features/Browser/ViewModels/Leaving/LeaveSandboxViewModel.h"
#include "Internationalization/Internationalization.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "FAskUserToPersistView"

namespace UE::SandboxedEditing
{
FAskUserToPersistView::FAskUserToPersistView(const TSharedRef<FLeaveSandboxViewModel>& InLeaveViewModel)
	: LeaveViewModel(InLeaveViewModel)
{
	LeaveViewModel->OnAskUserToPersistWorkflow().AddRaw(this, &FAskUserToPersistView::ShowNotification);
}

FAskUserToPersistView::~FAskUserToPersistView()
{
	LeaveViewModel->OnAskUserToPersistWorkflow().RemoveAll(this);
}

void FAskUserToPersistView::ShowNotification(FAskUserToPersistWorkflow& InWorkflow) const
{
	USandboxedEditingSettings const* CurrentSettings = USandboxedEditingSettings::Get();
	if (CurrentSettings && !CurrentSettings->bAskToPersistWhenLeavingSandbox)
	{
		// The user has indicated that they don't want to be prompted for saving a sandbox on existing the sandbox.
		InWorkflow.LeaveWithoutPersist(); 
		return;
	}
	
	const EAppReturnType::Type Result = FMessageDialog::Open(
		EAppMsgType::YesNoCancel, EAppReturnType::No,
		LOCTEXT("Question", "You are about to leave a sandbox containing changes. Do you want to persist the changes?")
		);
	switch (Result)
	{
	case EAppReturnType::Yes: InWorkflow.ProceedToPersist(); break;
	case EAppReturnType::No: InWorkflow.LeaveWithoutPersist(); break;
	case EAppReturnType::Cancel: InWorkflow.CancelLeave(); break;
	default: checkNoEntry(); InWorkflow.CancelLeave();
	}
}
}

#undef LOCTEXT_NAMESPACE
