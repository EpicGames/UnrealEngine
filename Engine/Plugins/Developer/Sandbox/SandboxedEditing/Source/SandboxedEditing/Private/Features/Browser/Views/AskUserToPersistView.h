// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::SandboxedEditing
{
class FAskUserToPersistWorkflow;
class FLeaveSandboxViewModel;

/** Visualizes FAskUserToPersistWorkflow by displaying a dialogue. */
class FAskUserToPersistView
{
public:
	
	explicit FAskUserToPersistView(const TSharedRef<FLeaveSandboxViewModel>& InLeaveViewModel);
	~FAskUserToPersistView();
	
private:
	
	/** Notifies when FAskUserToPersistWorkflow starts.  */
	const TSharedRef<FLeaveSandboxViewModel> LeaveViewModel;

	void ShowNotification(FAskUserToPersistWorkflow& InWorkflow) const;
};
}
