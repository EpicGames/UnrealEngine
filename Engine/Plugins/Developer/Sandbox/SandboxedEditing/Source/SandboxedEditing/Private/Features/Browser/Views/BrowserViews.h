// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AskUserAboutDirtyPackagesView.h"
#include "AskUserToPersistView.h"
#include "PersistSandboxView.h"
#include "RenameWorkflowView.h"
#include "Templates/SharedPointer.h"
#include "Transfer/ExportView.h"
#include "Transfer/ImportView.h"

namespace UE::SandboxedEditing
{
class FLeaveSandboxViewModel;

/** 
 * Holds the views for Sandboxed Editing. This is mostly for handling displaying notifications, spawning dialogues, etc. requested by view models.
 * @note Only contains "explicit" views. Of course, widgets, e.g. SSandboxedEditingRoot, implicitly classify as views. Those are not held here though.
 */
struct FBrowserViews
{
	/** When leaving with in-memory changes, this displays a toaster. */
	const TSharedRef<FAskUserAboutDirtyPackagesView> AskUserAboutDirtyPackagesView;
	
	/** When leaving, this asks the user whether they want to persist files. */
	const TSharedRef<FAskUserToPersistView> AskUserToPersistView;
	
	/** Visualizes FPersistSandboxWorkflow. */
	const TSharedRef<FPersistSandboxView> PersistSandboxView;
	
	/** Displays a notification when a sandbox is renamed. */
	const TSharedRef<FRenameWorkflowView> RenameWorkflowView;
	
	/** Presents the UI for exporting sandboxes. */
	const TSharedRef<FExportView> ExportView;
	/** Presents the UI for importing sandboxes. */
	const TSharedRef<FImportView> ImportView;
	
	explicit FBrowserViews(
		const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel, 
		const TSharedRef<FPersistSandboxViewModel>& InPersistViewModel, 
		const TSharedRef<FLeaveSandboxViewModel>& InLeaveViewModel
		)
		: AskUserAboutDirtyPackagesView(MakeShared<FAskUserAboutDirtyPackagesView>(InLeaveViewModel))
		, AskUserToPersistView(MakeShared<FAskUserToPersistView>(InLeaveViewModel))
		, PersistSandboxView(MakeShared<FPersistSandboxView>(InPersistViewModel))
		, RenameWorkflowView(MakeShared<FRenameWorkflowView>(InControlsViewModel))
		, ExportView(MakeShared<FExportView>(InControlsViewModel))
		, ImportView(MakeShared<FImportView>(InControlsViewModel))
	{}
};
}
