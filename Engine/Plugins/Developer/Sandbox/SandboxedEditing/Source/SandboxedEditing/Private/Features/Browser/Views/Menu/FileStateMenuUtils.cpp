// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileStateMenuUtils.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Features/Browser/Commands/FileStateActions/FileStateCommands.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Features/Browser/Widgets/Shared/FileState/SFileStateListView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "FileStateMenu"

namespace UE::SandboxedEditing::FileStateMenu
{
TArray<FString> TransformSelectedItems(const TSharedRef<SFileStateListView>& InListView)
{
	const TArray<TSharedPtr<FFileStateItem>> SelectedItems = InListView->GetSelectedFiles();
	TArray<FString> NonSandboxFiles;
	Algo::Transform(SelectedItems, NonSandboxFiles, [](const TSharedPtr<FFileStateItem>& InItem)
	{
		return InItem->NonSandboxFile;
	});
	return NonSandboxFiles;
}

TAttribute<TArray<FString>> MakeSelectedFilesAttribute(const TSharedRef<SFileStateListView>& InListView)
{
	return TAttribute<TArray<FString>>::CreateLambda([WeakListView = InListView.ToWeakPtr()]
	{
		const TSharedPtr<SFileStateListView> ListViewPin = WeakListView.Pin();
		return ListViewPin ? TransformSelectedItems(ListViewPin.ToSharedRef()) : TArray<FString>{};
	});
}

namespace Detail
{
static FText GetExploreFolderText()
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("FileManagerName"), FPlatformMisc::GetFileManagerName());
	return FText::Format(NSLOCTEXT("GenericPlatform", "ShowInFileManager", "Show in {FileManagerName}"), Args);
}
}


void AppendMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection(TEXT("Explorer"), LOCTEXT("Explorer", "Explorer"));
	
	FFileStateCommands& Commands = FFileStateCommands::Get();
	InMenuBuilder.AddMenuEntry(Commands.BrowseToAsset);
	InMenuBuilder.AddMenuEntry(Commands.ShowRootInExplorer);
	InMenuBuilder.AddMenuEntry(Commands.ShowFileInExplorer, NAME_None, Detail::GetExploreFolderText());
	
	InMenuBuilder.EndSection();
}
}

#undef LOCTEXT_NAMESPACE