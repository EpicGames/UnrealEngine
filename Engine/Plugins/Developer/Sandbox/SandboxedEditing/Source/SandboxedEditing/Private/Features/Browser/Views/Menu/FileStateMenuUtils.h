// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class FMenuBuilder;
class FUICommandList;
namespace UE::SandboxedEditing { class SFileStateListView; }
template<typename ObjectType> class TAttribute;

namespace UE::SandboxedEditing::FileStateMenu
{
/** @return Gets the non-sandbox file paths selected in InListView. */
TArray<FString> TransformSelectedItems(const TSharedRef<SFileStateListView>& InListView);
/** @return Attribute returning result of TransformSelectedItems. */
TAttribute<TArray<FString>> MakeSelectedFilesAttribute(const TSharedRef<SFileStateListView>& InListView);

/** Appends entries that are shared across all menus. */
void AppendMenu(FMenuBuilder& InMenuBuilder);
}
