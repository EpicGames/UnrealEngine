// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistSandboxView.h"

#include "Features/Browser/ViewModels/Persist/PersistOperationViewModel.h"
#include "Features/Browser/ViewModels/Persist/PersistSandboxViewModel.h"
#include "Features/Browser/Widgets/Sandboxed/SPersistOperationWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/SWindow.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FPersistSandboxView"

namespace UE::SandboxedEditing
{
FPersistSandboxView::FPersistSandboxView(const TSharedRef<FPersistSandboxViewModel>& InPersistViewModel)
	: PersistViewModel(InPersistViewModel)
{
	PersistViewModel->OnStartPersistWorkflow().AddRaw(this, &FPersistSandboxView::ShowPersistWorkflow);
	PersistViewModel->OnRequestPersistSummaryNotification().AddRaw(this, &FPersistSandboxView::ShowPersistSummary);
}

FPersistSandboxView::~FPersistSandboxView()
{
	PersistViewModel->OnStartPersistWorkflow().RemoveAll(this);
	PersistViewModel->OnRequestPersistSummaryNotification().RemoveAll(this);
}

namespace PersistViewDetail
{
static void PreSelectFiles(FPersistOperationViewModel& InViewModel, TConstArrayView<FString> InPreSelectedFiles)
{
	if (!InPreSelectedFiles.IsEmpty())
	{
		InViewModel.SetAllFilesPersisted(false);
		
		const TArray<FString>& NonSandboxPaths = InViewModel.GetPersistableFiles().NonSandboxPaths;
		for (const FString& PreSelected : InPreSelectedFiles)
		{
			const int32 Index = NonSandboxPaths.IndexOfByPredicate([&PreSelected](const FString& Path)
			{
				return FPaths::IsSamePath(Path, PreSelected);
			});
			
			if (ensure(NonSandboxPaths.IsValidIndex(Index)))
			{
				InViewModel.SetFilePersisted(Index, true);
			}
		}
	}
}
}

void FPersistSandboxView::ShowPersistWorkflow(FPersistSandboxWorkflow& InWorkflow, TConstArrayView<FString> InPreSelectedFiles) const
{
	const TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(LOCTEXT("PersistSubmitWindowTitle", "Persist Sandbox"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600, 600))
		.SupportsMaximize(true)
		.SupportsMinimize(false);
	
	const TSharedRef<FPersistOperationViewModel> ViewModel = MakeShared<FPersistOperationViewModel>(InWorkflow);
	PersistViewDetail::PreSelectFiles(*ViewModel, InPreSelectedFiles);
	NewWindow->SetContent(
		SNew(SPersistOperationWidget, ViewModel)
	);
	
	InWorkflow.OnWorkflowEnded().AddLambda([WeakWindow = NewWindow.ToWeakPtr()]
	{
		const TSharedPtr<SWindow> WindowPin = WeakWindow.Pin();
		WindowPin->RequestDestroyWindow();
	});
	FSlateApplication::Get().AddModalWindow(NewWindow, nullptr);
	
	// If the user just closed the window, treat it as cancelling the workflow.
	// The workflow may be destroyed as the window may have confirmed the persist operation.
	if (FPersistSandboxWorkflow* Workflow = PersistViewModel->GetPersistWorkflow())
	{
		Workflow->CancelPersist();
	}
}

namespace PersistViewDetail
{
static FText BuildErrorSubtext(const FileSandboxUI::FPersistSummary& InSummary)
{
	return FText::Format(LOCTEXT("Persist.SubText.WithErrorsFmt", "{0} persisted. {1} {0}|plural(one=error,other=errors).\nSee log for details."), 
		InSummary.Edited + InSummary.Added + InSummary.Deleted,
		InSummary.FailedFiles.Num()
		);
}
static FText BuildSuccessSubtext(const FileSandboxUI::FPersistSummary& InSummary)
{
	return FText::Format(
		LOCTEXT("Persist.SubText.SuccessFmt", "{0}{1}{2}"),
		InSummary.Edited > 0 ? FText::Format(LOCTEXT("Edited", "Edited {0} {0}|plural(one=file,other=files).\n"), InSummary.Edited) : FText::GetEmpty(),
		InSummary.Added > 0 ? FText::Format(LOCTEXT("Added", "Added {0} {0}|plural(one=file,other=files).\n"), InSummary.Added) : FText::GetEmpty(),
		InSummary.Deleted > 0 ? FText::Format(LOCTEXT("Removed", "Removed {0} {0}|plural(one=file,other=files)."), InSummary.Deleted) : FText::GetEmpty()
		);
}
}

void FPersistSandboxView::ShowPersistSummary(const FileSandboxUI::FPersistSummary& InSummary) const
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();

	const bool bHasErrors = !InSummary.FailedFiles.IsEmpty();
	const FText Title = bHasErrors 
		? LOCTEXT("Persist.Title.WithErrors", "Persist completed with errors")
		: LOCTEXT("Persist.Title.Success", "Persist successful");
	
	FNotificationInfo Info(Title);
	Info.bFireAndForget = false;
	Info.SubText = bHasErrors ? PersistViewDetail::BuildErrorSubtext(InSummary) : PersistViewDetail::BuildSuccessSubtext(InSummary);
	
	const TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Info);
	Item->SetCompletionState(bHasErrors ? SNotificationItem::CS_Fail : SNotificationItem::CS_Success);
	Item->RegisterActiveTimer(3.f, FWidgetActiveTimerDelegate::CreateLambda([WeakItem = Item.ToWeakPtr()](auto, auto)
	{
		if (const TSharedPtr<SNotificationItem> ItemPin = WeakItem.Pin())
		{
			ItemPin->ExpireAndFadeout();
		}
		return EActiveTimerReturnType::Stop;
	}));
}
}

#undef LOCTEXT_NAMESPACE