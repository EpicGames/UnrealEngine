// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/Persist/PersistSandboxWorkflow.h"
#include "Persist/Feedback/PersistSummary.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FPersistSandboxViewModel;

/** Visualizes FPersistSandboxWorkflow. */
class FPersistSandboxView : public FNoncopyable
{
public:
	explicit FPersistSandboxView(const TSharedRef<FPersistSandboxViewModel>& InPersistViewModel);
	~FPersistSandboxView();
	
private:
	
	/** Notifies when FPersistSandboxWorkflow starts. */
	const TSharedRef<FPersistSandboxViewModel> PersistViewModel;
	
	/** Shows a modal in which the user can select the files to persist. */
	void ShowPersistWorkflow(FPersistSandboxWorkflow& InWorkflow, TConstArrayView<FString> InPreSelectedFiles) const;
	/** Shows a notification summarizing the result of the operation. */
	void ShowPersistSummary(const FileSandboxUI::FPersistSummary& InSummary) const;
};
}

