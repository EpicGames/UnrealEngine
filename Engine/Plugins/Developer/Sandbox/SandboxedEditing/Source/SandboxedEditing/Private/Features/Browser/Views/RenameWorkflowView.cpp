// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenameWorkflowView.h"

#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FRenameWorkflowView"

namespace UE::SandboxedEditing
{
FRenameWorkflowView::FRenameWorkflowView(const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel)
	: ControlsViewModel(InControlsViewModel)
{
	ControlsViewModel->OnRenameWorkflowStarted().AddRaw(this, &FRenameWorkflowView::OnStartRenameOperation);
}

FRenameWorkflowView::~FRenameWorkflowView()
{
	ControlsViewModel->OnRenameWorkflowStarted().RemoveAll(this);
	if (FRenameSandboxWorkflow* Workflow = ControlsViewModel->GetCurrentRenameWorkflow())
	{
		Workflow->OnSandboxRenamed().RemoveAll(this);
	}
}

void FRenameWorkflowView::OnStartRenameOperation(FRenameSandboxWorkflow& InWorkflow) const
{
	InWorkflow.OnSandboxRenamed().AddRaw(this, &FRenameWorkflowView::OnSandboxRenamed);
}

void FRenameWorkflowView::OnSandboxRenamed(const FString& InOldName, const FString& InNewName) const
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	const FText Title = FText::Format(
		LOCTEXT("Renamed", "Renamed sandbox \"{0}\" as \"{1}\""),
		FText::AsCultureInvariant(InOldName), 
		FText::AsCultureInvariant(InNewName)
		);
	
	NotificationManager.AddNotification(Title)
		->SetCompletionState(SNotificationItem::CS_Success);
}
}

#undef LOCTEXT_NAMESPACE