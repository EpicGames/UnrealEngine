// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/RenameSandboxWorkflow.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxControlsViewModel;

/** Displays a notification when a sandbox is renamed. */
class FRenameWorkflowView : public FNoncopyable
{
public:
	
	explicit FRenameWorkflowView(const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel);
	~FRenameWorkflowView();
	
private:
	
	const TSharedRef<FSandboxControlsViewModel> ControlsViewModel;

	/** Subscribes OnSandboxRenamed. */
	void OnStartRenameOperation(FRenameSandboxWorkflow& InWorkflow) const;
	/** Shows a slate notification that the sandbox was renamed. */
	void OnSandboxRenamed(const FString& InOldName, const FString& InNewName) const;
};
}

