// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExportView.h"

#include "DesktopPlatformModule.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Features/Browser/ViewModels/Transfer/ExportWorkflowResult.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FExportView"

namespace UE::SandboxedEditing
{
FExportView::FExportView(const TSharedRef<FSandboxControlsViewModel>& InViewModel)
	: ViewModel(InViewModel)
{
	ViewModel->OnExportWorkflowStarted().AddRaw(this, &FExportView::OnExportWorkflowStarted);
}

FExportView::~FExportView()
{
	// This cleanup should not matter effectively, the point is to follow proper RAII out of principle.
	ViewModel->OnExportWorkflowStarted().RemoveAll(this);
	
	if (FExportWorkflow* Workflow = ViewModel->GetCurrentExportWorkflow())
	{
		Workflow->OnExportEnded().RemoveAll(this);
	}
}

namespace ExportViewDetail
{
static TOptional<FString> AskUserForZipPath(FExportWorkflow& InWorkflow, IDesktopPlatform& InDesktopPlatform)
{
	TArray<FString> SelectedFileNames;
	const bool bSuccess = InDesktopPlatform.SaveFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		LOCTEXT("ExportSandbox", "Export Sandbox").ToString(),
		InWorkflow.GetDefaultExportDirectory(),
		InWorkflow.GetDefaultSandboxFileName(),
		TEXT("*.zip"),
		EFileDialogFlags::None,
		SelectedFileNames
		);
	return bSuccess && SelectedFileNames.Num() == 1 ? SelectedFileNames[0] : TOptional<FString>();
}

static TOptional<FString> AskUserForDirectoryPath(FExportWorkflow& InWorkflow, IDesktopPlatform& InDesktopPlatform)
{
	FString SelectedDirectory;
	const bool bSuccess = InDesktopPlatform.OpenDirectoryDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		LOCTEXT("ExportSandboxes", "Export Sandboxes").ToString(),
		InWorkflow.GetDefaultExportDirectory(),
		SelectedDirectory
		);
	if (!bSuccess)
	{
		return TOptional<FString>();
	}
	
	const TArray<FString> OverridenFiles = InWorkflow.GetFilesOverridenByExportDirectory(SelectedDirectory);
	if (OverridenFiles.IsEmpty())
	{
		return SelectedDirectory;
	}
	
	const FString FilesList = FString::JoinBy(OverridenFiles, TEXT("\n- "), [](const FString& FilePath)
	{
		return FPaths::GetCleanFilename(FilePath);
	});
	const FText Message = FText::Format(
		LOCTEXT("Directory.OverrideFiles.Message", "The following files in {0} will be overwritten:\n- {1}\n\nDo you want to continue?"), 
		FText::AsCultureInvariant(SelectedDirectory),
		FText::AsCultureInvariant(FilesList)
		);
	if (FMessageDialog::Open(EAppMsgType::OkCancel, Message, LOCTEXT("Directory.OverrideFiles.Title", "Override files")) == EAppReturnType::Ok)
	{
		return SelectedDirectory;
	}
	
	return TOptional<FString>();
}

static TOptional<FString> AskUserForExportPath(FExportWorkflow& InWorkflow)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!ensure(DesktopPlatform))
	{
		return {};
	}
	
	static_assert(static_cast<int32>(EExportPathType::Count) == 2, "Update this logic.");
	switch (InWorkflow.GetExportType())
	{
	case EExportPathType::Directory: return AskUserForDirectoryPath(InWorkflow, *DesktopPlatform);
	case EExportPathType::Zip: return AskUserForZipPath(InWorkflow, *DesktopPlatform);
	default: checkNoEntry(); return {};
	}
}
}

void FExportView::OnExportWorkflowStarted(FExportWorkflow& InWorkflow)
{
	InWorkflow.OnExportEnded().AddRaw(this, &FExportView::OnExportEnded);
	
	if (const TOptional<FString> ExportPath = ExportViewDetail::AskUserForExportPath(InWorkflow))
	{
		InWorkflow.SetExportPath(*ExportPath);
		InWorkflow.TryExportOrCancel();
	}
	else
	{
		InWorkflow.Cancel();
	}
}

namespace ExportViewDetail
{
constexpr float FadeOutTime = 4.f;

static void ShowSuccess(TConstArrayView<FString> InExportedSandboxes, const FString& InExportPath)
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	const bool bOnlyOne = InExportedSandboxes.Num() == 1;
	const FText Title = bOnlyOne 
		? FText::Format(
			LOCTEXT("Success.SingleFmt", "Exported {0}"),
			FText::AsCultureInvariant(SandboxModel::GetSandboxName(InExportedSandboxes[0]).Get(TEXT("sandbox")))
			)
		: FText::Format(LOCTEXT("Success.MultipleFmt", "Exported {0} {0}|plural(one=sandbox,other=sandboxes)"), InExportedSandboxes.Num());
	
	FNotificationInfo NotificationInfo(Title);
	NotificationInfo.SubText = bOnlyOne 
		? FText::Format(LOCTEXT("Success.SingleSubtextFmt", "Exported to {0}"), FText::AsCultureInvariant(InExportPath)) 
		: FText::GetEmpty();
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Success);
}

static void ShowWithErrors(const FExportWorkflowResult& InResult, int32 InNumExported)
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo NotificationInfo(LOCTEXT("WithErrors.Title", "Exported with errors"));
	NotificationInfo.SubText = FText::Format(
		LOCTEXT("WithErrors.SubtextFmt", "Exported {0} {0}|plural(one=sandbox,other=sandboxes).\nFailed to export {1}{1}|plural(one=sandbox,other=sandboxes)"), 
		InNumExported - InResult.SandboxesWithErrors.Num(),
		InResult.SandboxesWithErrors.Num()
		);
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
}

static void ShowCancel()
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo NotificationInfo(LOCTEXT("Cancelled", "Cancelled sandbox export"));
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo);
}
}

void FExportView::OnExportEnded(const FExportWorkflowResult& InResult)
{
	const FExportWorkflow* Workflow = ViewModel->GetCurrentExportWorkflow();
	check(Workflow);
	
	static_assert(static_cast<int32>(EExportWorkflowResult::Count) == 3, "Update this switch");
	switch (InResult.ExportResult)
	{
	case EExportWorkflowResult::Success: ExportViewDetail::ShowSuccess(Workflow->GetExportedSandboxRootPaths(), Workflow->GetExportPath()); break;
	case EExportWorkflowResult::SomeErrors: ExportViewDetail::ShowWithErrors(InResult, Workflow->GetExportedSandboxRootPaths().Num()); break;
	case EExportWorkflowResult::Cancelled: ExportViewDetail::ShowCancel(); break;
	default: break;
	}
}
}

#undef LOCTEXT_NAMESPACE
