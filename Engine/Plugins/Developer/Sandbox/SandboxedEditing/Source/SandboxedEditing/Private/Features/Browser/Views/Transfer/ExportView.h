// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/Transfer/ExportWorkflow.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxControlsViewModel;

/** Presents dialogues for exporting sandboxes. */
class FExportView : public FNoncopyable
{
public:
	
	explicit FExportView(const TSharedRef<FSandboxControlsViewModel>& InViewModel);
	~FExportView();
	
private:
	
	/** Produces the export workflow that the view binds to. */
	const TSharedRef<FSandboxControlsViewModel> ViewModel;

	void OnExportWorkflowStarted(FExportWorkflow& InWorkflow);
	void OnExportEnded(const FExportWorkflowResult& InResult);
};
}

