// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportView.h"

#include "DesktopPlatformModule.h"
#include "IDesktopPlatform.h"
#include "Algo/RemoveIf.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Features/Browser/ViewModels/Transfer/ImportWorkflowResult.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FImportView"

namespace UE::SandboxedEditing
{
FImportView::FImportView(const TSharedRef<FSandboxControlsViewModel>& InViewModel)
	: ViewModel(InViewModel)
{
	ViewModel->OnImportWorkflowStarted().AddRaw(this, &FImportView::OnImportWorkflowStarted);
}

FImportView::~FImportView()
{
	// This cleanup should not matter effectively, the point is to follow proper RAII out of principle.
	ViewModel->OnImportWorkflowStarted().RemoveAll(this);
	
	if (FImportWorkflow* Workflow = ViewModel->GetCurrentImportWorkflow())
	{
		Workflow->OnImportEnded().RemoveAll(this);
	}
}

namespace ImportViewDetail
{
static TArray<FString> AskUserForFilesToImport(FImportWorkflow& InWorkflow)
{
	TArray<FString> Result;
	
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!ensure(DesktopPlatform))
	{
		return Result;
	}
	
	DesktopPlatform->OpenFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ImportSandbox", "Import Sandbox").ToString(),
			InWorkflow.GetDefaultImportDirectory(),
			TEXT(""),
			TEXT("*.zip"),
			EFileDialogFlags::Multiple,
			Result
		);
	return Result;
}

/** @return List of sandbox names */
static FString MakeSandboxNameList(const TArray<FString>& InStrings)
{
	return FString::JoinBy(InStrings, TEXT("\n"), [](const FString& SandboxPath)
	{
		const FString& PathLeaf = FPaths::GetPathLeaf(SandboxPath);
		return FString::Printf(TEXT(" - %s"), *PathLeaf);
	});
}

static void ShowInvalidZipSelected(const TArray<FString>& InFilesToImport, const TArray<FString>& InInvalidZips)
{
	const FText AllInvalid = FText::Format(
		LOCTEXT("Invalid.Message.All", "The selected {0}|plural(one=file does,other=files do) not contain any {0}|plural(one=sandbox,other=sandboxes)."),
		InFilesToImport.Num()
		);
	
	const FText SomeInvalid = FText::Format(
		LOCTEXT("Invalid.Message.Some", "The following {0}|plural(one=file does,other=files do) not contain any sandbox."),
		InInvalidZips.Num(),
		FText::AsCultureInvariant(MakeSandboxNameList(InInvalidZips))
		);
	
	FMessageDialog::Open(
		EAppMsgType::Ok, 
		InFilesToImport.Num() == 1 || InFilesToImport.Num() == InInvalidZips.Num() ? AllInvalid : SomeInvalid,
		LOCTEXT("Invalid.Title", "Cannot import sandboxes")
		);
}

static void AskUserToOverwriteAndProceed(
	FImportWorkflow& InWorkflow, TArray<FString>& InFilesToImport, const FImportValidationResult InValidationResult
	)
{
	const FText Sandboxes = FText::AsCultureInvariant(MakeSandboxNameList(InValidationResult.AskForOverwrite));
	const EAppReturnType::Type OverwriteFilesResponse = FMessageDialog::Open(
		EAppMsgType::YesNoCancel,
		FText::Format(LOCTEXT("Overwrite.MessageFmt", "The following sandboxes already exist:\n{0}\n\nDo you want to overwrite them?"), Sandboxes),
		LOCTEXT("Overwrite.Title", "Overwrite sandboxes")
		);
	
	switch (OverwriteFilesResponse)
	{
	case EAppReturnType::No:
		InFilesToImport.SetNum(Algo::RemoveIf(InFilesToImport, [&InValidationResult](const FString& InFile)
		{
			return InValidationResult.AskForOverwrite.Contains(InFile);
		}));
		InWorkflow.SetFilesToImport(InFilesToImport);
		InWorkflow.TryImportOrCancel();
		break;
	case EAppReturnType::Yes: 
		InWorkflow.TryImportOrCancel();
		break;
	case EAppReturnType::Cancel:
		InWorkflow.Cancel();
		break;
	default: checkNoEntry();
	}
}
}

void FImportView::OnImportWorkflowStarted(FImportWorkflow& InWorkflow)
{
	InWorkflow.OnImportEnded().AddRaw(this, &FImportView::OnImportEnded);
	
	TArray<FString> FilesToImport = ImportViewDetail::AskUserForFilesToImport(InWorkflow);
	if (FilesToImport.IsEmpty())
	{
		InWorkflow.Cancel();
		return;
	}
	
	const FImportValidationResult ValidationResult = InWorkflow.SetFilesToImport(FilesToImport);
	const TArray<FString>& InvalidZips = ValidationResult.InvalidZips;
	if (!InvalidZips.IsEmpty())
	{
		ImportViewDetail::ShowInvalidZipSelected(FilesToImport, InvalidZips);
		InWorkflow.Cancel();
		return;
	}
	
	if (!ValidationResult.AskForOverwrite.IsEmpty())
	{
		ImportViewDetail::AskUserToOverwriteAndProceed(InWorkflow, FilesToImport, ValidationResult);
		return;
	}
	
	InWorkflow.TryImportOrCancel();
}

namespace ImportViewDetail
{
constexpr float FadeOutTime = 4.f;

static void ShowSuccess(TConstArrayView<FString> InImportedSandboxed)
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	const bool bOnlyOne = InImportedSandboxed.Num() == 1;
	const FText Title = bOnlyOne 
		? FText::Format(
			LOCTEXT("Success.SingleFmt", "Imported {0}"),
			FText::AsCultureInvariant(SandboxModel::GetSandboxName(InImportedSandboxed[0]).Get(TEXT("sandbox")))
			)
		: FText::Format(LOCTEXT("Success.MultipleFmt", "Imported {0} {0}|plural(one=sandbox,other=sandboxes)"), InImportedSandboxed.Num());
	
	FNotificationInfo NotificationInfo(Title);
	NotificationInfo.SubText = bOnlyOne 
		? FText::Format(LOCTEXT("Success.SingleSubtextFmt", "Imported from {0}"), FText::AsCultureInvariant(InImportedSandboxed[0])) 
		: FText::GetEmpty();
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Success);
}

static void ShowWithErrors(const FImportWorkflowResult& InResult, TConstArrayView<FString> InFilesToImport)
{
	const bool bOnlyOne = InFilesToImport.Num() == 1;
	const FString& FirstSandboxName = FPaths::GetPathLeaf(InFilesToImport[0]);
	
	const FText Title = bOnlyOne 
		? FText::Format(LOCTEXT("WithErrors.TitleOnlyOne", "Failed to import {0}"), FText::AsCultureInvariant(FirstSandboxName)) 
		: LOCTEXT("WithErrors.Title", "Imported with errors");
	const FText SubText = bOnlyOne 
		? FText::GetEmpty()
		: FText::Format(
			LOCTEXT("WithErrors.SubtextFmt", "Imported {0} {0}|plural(one=sandbox,other=sandboxes).\nFailed to import {1}{1}|plural(one=sandbox,other=sandboxes)"), 
			InFilesToImport.Num() - InResult.Errors.Num(),
			InResult.Errors.Num()
			);
	
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo NotificationInfo(Title);
	NotificationInfo.SubText = SubText;
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
}

static void ShowCancel()
{
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo NotificationInfo(LOCTEXT("Cancelled", "Cancelled sandbox import"));
	NotificationInfo.ExpireDuration = FadeOutTime;
	NotificationInfo.FadeOutDuration = FadeOutTime;
	NotificationManager.AddNotification(NotificationInfo);
}
}

void FImportView::OnImportEnded(const FImportWorkflowResult& InImportResult)
{
	const FImportWorkflow* Workflow = ViewModel->GetCurrentImportWorkflow();
	check(Workflow);
	
	static_assert(static_cast<int32>(EImportWorkflowResult::Count) == 3, "Update this switch");
	switch (InImportResult.ResultCode)
	{
	case EImportWorkflowResult::Success: ImportViewDetail::ShowSuccess(Workflow->GetFilesToImport()); break;
	case EImportWorkflowResult::SomeErrors: ImportViewDetail::ShowWithErrors(InImportResult, Workflow->GetFilesToImport()); break;
	case EImportWorkflowResult::Cancelled: ImportViewDetail::ShowCancel(); break;
	default: checkNoEntry();
	}
}
}

#undef LOCTEXT_NAMESPACE