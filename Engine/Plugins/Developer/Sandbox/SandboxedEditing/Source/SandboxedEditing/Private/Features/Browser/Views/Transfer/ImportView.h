// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/Transfer/ImportWorkflow.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FImportWorkflow;
class FSandboxControlsViewModel;
struct FImportWorkflowResult;

/** Presents dialogues for importing sandboxes. */
class FImportView : public FNoncopyable
{
public:
	
	explicit FImportView(const TSharedRef<FSandboxControlsViewModel>& InViewModel);
	~FImportView();
	
private:
	
	/** Produces the import workflow that the view binds to. */
	const TSharedRef<FSandboxControlsViewModel> ViewModel;
	
	void OnImportWorkflowStarted(FImportWorkflow& InWorkflow);
	void OnImportEnded(const FImportWorkflowResult& InImportResult);
};
}

