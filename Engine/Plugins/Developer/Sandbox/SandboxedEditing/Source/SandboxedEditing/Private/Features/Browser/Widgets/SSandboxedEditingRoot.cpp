// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxedEditingRoot.h"

#include "EntryPoint/EntryPointWidgetFactory.h"
#include "EntryPoint/IExternalSandboxActiveViewModel.h"
#include "Features/Browser/Commands/BrowserCommandBindings.h"
#include "Features/Browser/ViewModels/Active/ActiveSandboxTrackerViewModel.h"
#include "Features/Browser/ViewModels/BrowserViewModels.h"
#include "Features/Browser/Widgets/Unsandboxed/SSandboxedEditingBrowser.h"
#include "Sandboxed/SActiveSandbox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"

namespace UE::SandboxedEditing
{
void SSandboxedEditingRoot::Construct(const FArguments& InArgs, const FBrowserViewModels& InViewModels)
{
	CommandBindings = InArgs._CommandList;
	ActiveSandboxTrackerViewModel = InViewModels.ActiveSandboxTrackerViewModel;
	ExternalSandboxViewModel = InViewModels.ExternalSandboxViewModel;
	ActiveSandboxViewModels.Emplace(
		InViewModels.ActiveSandboxDetailsViewModel, InViewModels.MetaDataViewModel, 
		InViewModels.ActiveSandboxFileStateViewModel, InViewModels.FilterActiveSandboxFileStateViewModel,
		InViewModels.PersistViewModel
		);
	ActiveFileStateColumnsAttr = InArgs._ActiveFileStateColumnsFactories;
	
	ActiveSandboxTrackerViewModel->OnLoadSandbox().AddSP(this, &SSandboxedEditingRoot::UpdateWidgetContent);
	ActiveSandboxTrackerViewModel->OnLeaveSandbox().AddSP(this, &SSandboxedEditingRoot::ShowBrowser);
	ExternalSandboxViewModel->OnVisibilityChanged().AddSP(this, &SSandboxedEditingRoot::UpdateWidgetContent);
	
	const SSandboxedEditingBrowser::FViewModels ViewModels 
	{ 
		InViewModels.FilterSandboxViewModel, InViewModels.ListViewModel, InViewModels.ControlsViewModel, InViewModels.MetaDataViewModel,
		InViewModels.UnloadedSandboxFileStateViewModel, InViewModels.FilterUnloadedSandboxFileStateViewModel
	};
	Browser = SNew(SSandboxedEditingBrowser, ViewModels)
		.CommandList(CommandBindings)
		.SandboxColumnFactories(InArgs._SandboxColumnFactories)
		.FileActionsColumnFactories(InArgs._BrowserFileActionsColumnFactories);
	
	UpdateWidgetContent();
}

FReply SSandboxedEditingRoot::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandBindings && CommandBindings->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SSandboxedEditingRoot::UpdateWidgetContent()
{
	if (ActiveSandboxTrackerViewModel->HasActiveSandbox())
	{
		ShowActiveSandbox();
	}
	else
	{
		ShowBrowser();
	}
}

void SSandboxedEditingRoot::ShowActiveSandbox()
{
	ChildSlot
	[
		SNew(SActiveSandbox, *ActiveSandboxViewModels)
		.CommandList(CommandBindings)
		.ColumnFactories(ActiveFileStateColumnsAttr.Get())
	];
}

void SSandboxedEditingRoot::ShowBrowser()
{
	ChildSlot
	[
		SNew(SOverlay)

		+SOverlay::Slot()
		[
			Browser.ToSharedRef()
		]
		
		+SOverlay::Slot()
		[
			SNew(SBox)
			.Visibility_Lambda([this]{ return ExternalSandboxViewModel->IsExternalSandboxActive() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				FileSandboxUI::MakeExternalSandboxActiveOverlay(ExternalSandboxViewModel.ToSharedRef())
			]
		]
	];
}
}
