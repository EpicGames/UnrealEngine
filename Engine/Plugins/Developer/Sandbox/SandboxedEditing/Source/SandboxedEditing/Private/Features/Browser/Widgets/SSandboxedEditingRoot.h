// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Sandboxed/SActiveSandbox.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
namespace UE::FileSandboxUI { class IExternalSandboxActiveViewModel; }

namespace UE::SandboxedEditing
{
class FActiveSandboxTrackerViewModel;
class FBrowserCommandBindings;
class FSandboxControlsViewModel;
class FSandboxListViewModel;
class FSandboxMetaDataViewModel;
class SSandboxedEditingBrowser;
struct FBrowserViewModels;

/** Root widget for Sandboxed Editing. It updates it content depending on whether the engine is sandboxed. */
class SSandboxedEditingRoot : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSandboxedEditingRoot){}
		/** The command list to bind selection-based commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		
		/** Used to build the columns. */
		SLATE_ARGUMENT(FSandboxColumnFactoryMap, SandboxColumnFactories)
		/** Used to build the columns for the file actions list displayed in the browser. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, BrowserFileActionsColumnFactories)
		
		/** Used to build the columns for the file actions list for the active sandbox. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ActiveFileStateColumnsFactories)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FBrowserViewModels& InViewModels);
	
	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface
	
private:
	
	/** The sandbox editing browser. Shown as long as not in any sandbox. */
	TSharedPtr<SSandboxedEditingBrowser> Browser;
	
	/** Knows when the active sandbox changes. */
	TSharedPtr<FActiveSandboxTrackerViewModel> ActiveSandboxTrackerViewModel;
	/** Knows when an external sandbox is active. */
	TSharedPtr<FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;
	
	/** The view models required to construct the active sandbox widget. Set once Construct has been called. */
	TOptional<SActiveSandbox::FViewModels> ActiveSandboxViewModels;
	/** Used to build the columns for the file actions list for the active sandbox. */
	TAttribute<FFileStateColumnFactoryMap> ActiveFileStateColumnsAttr;
	
	/** Command bindings for the browser UI. */
	TSharedPtr<FUICommandList> CommandBindings;
	
	/** Sets the widget content to the browser or active session widget depending on whether we're in a sandbox. */
	void UpdateWidgetContent();
	
	void ShowActiveSandbox();
	void ShowBrowser();
};
}

