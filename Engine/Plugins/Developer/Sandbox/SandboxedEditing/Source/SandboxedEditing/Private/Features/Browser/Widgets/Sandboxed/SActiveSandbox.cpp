// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveSandbox.h"

#include "SActiveSandboxFileChanges.h"
#include "Features/Browser/ViewModels/Active/ActiveSandboxDetailsViewModel.h"
#include "Features/Browser/ViewModels/FileState/Models/ActiveSandboxFileStateViewModel.h"
#include "Features/Browser/Widgets/Shared/FileState/SFilterableFileStateListView.h"
#include "Features/Browser/Widgets/Shared/SSandboxDescription.h"
#include "SActiveSandboxToolbar.h"
#include "Features/Browser/Commands/BrowserCommandBindings.h"
#include "Features/Browser/Commands/BrowserCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SActiveSandbox"

namespace UE::SandboxedEditing
{
void SActiveSandbox::Construct(const FArguments& InArgs, const FViewModels& InViewModels)
{
	ActiveSandboxViewModel = InViewModels.ActiveSandboxViewModel;
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(1)
			[
				SNew(SActiveSandboxToolbar, InViewModels.ActiveSandboxViewModel)
				.CommandList(InArgs._CommandList)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Thickness(FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorThickness)
				.SeparatorImage(&FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorBrush)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(1)
			[
				MakeDescription(InViewModels)
			]

			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(1)
			[
				MakeFileChangesContent(InArgs, InViewModels)
			]
		]
	];
}

TSharedRef<SWidget> SActiveSandbox::MakeDescription(const FViewModels& InViewModels)
{
	return SNew(SExpandableArea)
		.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Description.Label", "Description"))
		]
		.BodyContent()
		[
			SNew(SBox)
			.MinDesiredHeight(120)
			[
				SNew(SSandboxDescription, InViewModels.MetaDataViewModel)
				.SandboxPath(ActiveSandboxViewModel->GetSandboxPath())
			]
		];
}

TSharedRef<SWidget> SActiveSandbox::MakeFileChangesContent(const FArguments& InArgs, const FViewModels& InViewModels)
{
	const SActiveSandboxFileChanges::FViewModels ViewModels(
		InViewModels.ActiveSandboxViewModel, InViewModels.ActiveSandboxFileStateModel, InViewModels.FilterActiveSandboxFileStateModel,
		InViewModels.PersistViewModel
		);
	
	return SNew(SExpandableArea)
		.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FileChanges.Label", "Changes"))
			.ToolTipText(LOCTEXT("FileChanges.Tooltip", "The file changes made in this sandbox"))
		]
		.BodyContent()
		[
			SNew(SActiveSandboxFileChanges, ViewModels)
			.CommandList(InArgs._CommandList)
			.ColumnFactories(InArgs._ColumnFactories)
		];
}
}

#undef LOCTEXT_NAMESPACE