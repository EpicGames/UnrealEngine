// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FActiveSandboxDetailsViewModel;
class FActiveSandboxFileStateViewModel;
class FFilterFileStateViewModel;
class FPersistSandboxViewModel;
class FSandboxMetaDataViewModel;
class SFilterableFileStateListView;

/** Widget displayed while in a sandbox */
class SActiveSandbox : public SCompoundWidget
{
public:
	
	struct FViewModels
	{
		TSharedRef<FActiveSandboxDetailsViewModel> ActiveSandboxViewModel;
		TSharedRef<FSandboxMetaDataViewModel> MetaDataViewModel;
		TSharedRef<FActiveSandboxFileStateViewModel> ActiveSandboxFileStateModel;
		TSharedRef<FFilterFileStateViewModel> FilterActiveSandboxFileStateModel;
		TSharedRef<FPersistSandboxViewModel> PersistViewModel;

		explicit FViewModels(
			const TSharedRef<FActiveSandboxDetailsViewModel>& InActiveSandboxViewModel,
			const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel,
			const TSharedRef<FActiveSandboxFileStateViewModel>& InActiveSandboxFileStateModel,
			const TSharedRef<FFilterFileStateViewModel>& InFilterActiveSandboxFileStateModel,
			const TSharedRef<FPersistSandboxViewModel>& InPersistViewModel
			)
			: ActiveSandboxViewModel(InActiveSandboxViewModel)
			, MetaDataViewModel(InMetaDataViewModel)
			, ActiveSandboxFileStateModel(InActiveSandboxFileStateModel)
			, FilterActiveSandboxFileStateModel(InFilterActiveSandboxFileStateModel)
			, PersistViewModel(InPersistViewModel)
		{}
	};
	
	SLATE_BEGIN_ARGS(SActiveSandbox) {}
		/** The command list to bind commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		
		/** Used to build the columns for the file actions list. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FViewModels& InViewModels);
	
private:
	
	/** Details about the active sandbox. */
	TSharedPtr<FActiveSandboxDetailsViewModel> ActiveSandboxViewModel;

	/** Make the description widget. */
	TSharedRef<SWidget> MakeDescription(const FViewModels& InViewModels);
	
	/** Makes the widget that displays the files changed by the current sandbox. */
	TSharedRef<SWidget> MakeFileChangesContent(const FArguments& InArgs, const FViewModels& InViewModels);
};
}

