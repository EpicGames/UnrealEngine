// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveSandboxFileChanges.h"

#include "Features/Browser/Commands/FileStateActions/FileStateCommands.h"
#include "Features/Browser/ViewModels/Active/ActiveSandboxDetailsViewModel.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Features/Browser/ViewModels/Persist/PersistSandboxViewModel.h"
#include "Features/Browser/Views/Menu/FileStateMenuUtils.h"
#include "Features/Browser/Widgets/Shared/FileState/SFileStateListView.h"
#include "Features/Browser/Widgets/Shared/FileState/SFilterableFileStateListView.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SActiveSandboxFileChanges"

namespace UE::SandboxedEditing
{
namespace ActiveFileChangesDetail
{
/** 
 * TODO DP: This feature is experimental and not fully tested yet. What if you revert a change that another file depends on?
 * E.g. Revert adding a blueprint but a map has that BP saved inside of it?
 */
static TAutoConsoleVariable<bool> CVarAllowRevertUI(TEXT("SandboxedEditing.AllowRevertUI"), false, TEXT("Whether to allow the user to revert files."));
}

void SActiveSandboxFileChanges::Construct(
	const FArguments& InArgs, const FViewModels& InViewModels
	)
{
	PersistViewModel = InViewModels.PersistViewModel;
	CommandList = InArgs._CommandList;
	
	ChildSlot
	[
		SAssignNew(FileActionWidget, SFilterableFileStateListView, InViewModels.FileActionsViewModel, InViewModels.FilterViewModel)
		.ColumnFactories(InArgs._ColumnFactories)
		.NoChangesText(LOCTEXT("NoItems", "File changes appear here once you edit, add, or remove a file."))
		.OnContextMenuOpening(this, &SActiveSandboxFileChanges::MakeFileChangeContextMenu)	
	];
	
	BindCommands(InViewModels);
}

TSharedPtr<SWidget> SActiveSandboxFileChanges::MakeFileChangeContextMenu()
{
	FMenuBuilder Menu(true, CommandList);
	FFileStateCommands& FileStateCommands = FFileStateCommands::Get();

	Menu.BeginSection(TEXT("Actions"), LOCTEXT("FileActions", "File Actions"));
	Menu.AddMenuEntry(FileStateCommands.PersistSelected);
	if (ActiveFileChangesDetail::CVarAllowRevertUI.GetValueOnGameThread())
	{
		Menu.AddMenuEntry(
			LOCTEXT("Revert", "Revert"), 
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SActiveSandboxFileChanges::RevertSelection),
				FCanExecuteAction::CreateSP(this, &SActiveSandboxFileChanges::CanRevertSelection)
			));
	}
	Menu.EndSection();
	
	FileStateCommandBindings->AppendMenu(Menu);
	
	return Menu.MakeWidget();
}

void SActiveSandboxFileChanges::BindCommands(const FViewModels& InViewModels)
{
	FileStateCommandBindings = MakeUnique<FFileStateCommandBindings>(
		InViewModels.ActiveSandboxViewModel->GetSandboxPath(), 
		FileStateMenu::MakeSelectedFilesAttribute(FileActionWidget->GetListView().ToSharedRef())
		);
	
	if (!CommandList)
	{
		return;
	}
	
	FFileStateCommands& FileStateCommands = FFileStateCommands::Get();
	CommandList->MapAction(FileStateCommands.PersistSelected,
		FExecuteAction::CreateSP(this, &SActiveSandboxFileChanges::HandlePersistSelected),
		FCanExecuteAction::CreateSP(this, &SActiveSandboxFileChanges::CanPersistSelected)
		);
}

void SActiveSandboxFileChanges::HandlePersistSelected() const
{
	if (CanPersistSelected())
	{
		PersistViewModel->StartPersistWorkflowForActiveSandbox(
			FileStateMenu::TransformSelectedItems(FileActionWidget->GetListView().ToSharedRef())
			);
	}
}

bool SActiveSandboxFileChanges::CanPersistSelected() const
{
	return PersistViewModel->CanStartPersistWorkflowForActiveSandbox();
}

void SActiveSandboxFileChanges::RevertSelection() const
{
	if (CanRevertSelection())
	{
		const TArray<FString> Selection = FileStateMenu::TransformSelectedItems(FileActionWidget->GetListView().ToSharedRef());
		SandboxModel::RevertSpecifiedChanges(Selection);
	}
}

bool SActiveSandboxFileChanges::CanRevertSelection() const
{
	const TArray<FString> Selection = FileStateMenu::TransformSelectedItems(FileActionWidget->GetListView().ToSharedRef());
	return !Selection.IsEmpty() && SandboxModel::CanRevertSpecifiedChanges(Selection);
}
}

#undef LOCTEXT_NAMESPACE