// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/Commands/FileStateActions/FileStateCommandBindings.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FPersistSandboxViewModel;
class FActiveSandboxDetailsViewModel;
class FFilterFileStateViewModel;
class IFileStateViewModel;
class SFilterableFileStateListView;

/** Displays the file changes of the active sandbox. */
class SActiveSandboxFileChanges : public SCompoundWidget
{
public:
	
	struct FViewModels
	{
		TSharedRef<FActiveSandboxDetailsViewModel> ActiveSandboxViewModel;
		TSharedRef<IFileStateViewModel> FileActionsViewModel;
		TSharedRef<FFilterFileStateViewModel> FilterViewModel;
		TSharedRef<FPersistSandboxViewModel> PersistViewModel;

		explicit FViewModels(
			const TSharedRef<FActiveSandboxDetailsViewModel>& InActiveSandboxViewModel,
			const TSharedRef<IFileStateViewModel>& InFileActionsViewModel,
			const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel,
			const TSharedRef<FPersistSandboxViewModel>& InPersistViewModel
			)
			: ActiveSandboxViewModel(InActiveSandboxViewModel)
			, FileActionsViewModel(InFileActionsViewModel)
			, FilterViewModel(InFilterViewModel)
			, PersistViewModel(InPersistViewModel)
		{}
	};
	
	SLATE_BEGIN_ARGS(SActiveSandboxFileChanges) {}
		/** The command list to bind commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		
		/** Used to build the columns for the file actions list. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const FViewModels& InViewModels);
	
private:
	
	/** Used for building the right-click menu. */
	TSharedPtr<FUICommandList> CommandList;
	/** Used for binding right-click persist commands. */
	TSharedPtr<FPersistSandboxViewModel> PersistViewModel;
	
	/** Displays the file changes made in this sandbox. */
	TSharedPtr<SFilterableFileStateListView> FileActionWidget;
	
	/** Common entries for FileActionWidget. */
	TUniquePtr<FFileStateCommandBindings> FileStateCommandBindings;

	/** @return The context menu for FileActionWidget. */
	TSharedPtr<SWidget> MakeFileChangeContextMenu();

	/** Binds the commands that this widget introduces. */
	void BindCommands(const FViewModels& InViewModels);
	
	/** Brings up the persist dialogue with the current selection pre-selected. */
	void HandlePersistSelected() const;
	/** @return Whether the persist dialogue can be created at this time */
	bool CanPersistSelected() const;
	
	/** Handles command for reverting current selection. */
	void RevertSelection() const;
	/** Whether there is a selection, and it can be reverted. */
	bool CanRevertSelection() const;
};
}

