// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActiveSandboxToolbar.h"

#include "Features/Browser/Commands/BrowserCommands.h"
#include "Features/Browser/ViewModels/Active/ActiveSandboxDetailsViewModel.h"
#include "SandboxedEditingStyle.h"
#include "SPrimaryButton.h"
#include "Styling/AppStyle.h"
#include "Utils/WidgetUtils.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SActiveSandboxToolbar"

namespace UE::SandboxedEditing
{
void SActiveSandboxToolbar::Construct(const FArguments& InArgs, const TSharedRef<FActiveSandboxDetailsViewModel>& InActiveSandboxViewModel)
{
	ActiveSandboxViewModel = InActiveSandboxViewModel;
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(4.0f, 1.0f))
		[
			MakeNameWidget()
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4, 3)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		]
		
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(1.0f))
		[
			MakePrimaryButtonByCommand(
				InArgs._CommandList, FBrowserCommands::Get().PersistSandbox,
				LOCTEXT("PersistLabel", "Persist")
				)
		]

		// Move all the other content to the right if the panel is big
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]

		// Leave button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(1.0f))
		[
			MakeIconButtonByCommand(
				FSandboxedEditingStyle::Get().GetBrush("SandboxedEditing.Browser.LeaveSandbox"),
				InArgs._CommandList, FBrowserCommands::Get().LeaveSandbox, 
				TAttribute<FText>::CreateSP(this, &SActiveSandboxToolbar::GetLeaveButtonToolTipText)
				)
		]

		// Go to settings
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(1.0f))
		[
			CreateOpenSettingsButton()
		]
	];
}

TSharedRef<SWidget> SActiveSandboxToolbar::MakeNameWidget() const
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.ColorAndOpacity(FLinearColor(0.75f, 0.75f, 0.75f))
		.Padding(FMargin(0.0f, 4.0f, 6.0f, 4.0f))
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("BoldFont"))
			.Text_Lambda([this]{ return FText::AsCultureInvariant(ActiveSandboxViewModel->GetSandboxName()); })
		];
}

FText SActiveSandboxToolbar::GetLeaveButtonToolTipText() const
{
	FText Reason;
	return ActiveSandboxViewModel->CanLeaveSandbox(&Reason) 
		? FBrowserCommands::Get().LeaveSandbox->GetDescription() 
		: Reason;
}
}

#undef LOCTEXT_NAMESPACE