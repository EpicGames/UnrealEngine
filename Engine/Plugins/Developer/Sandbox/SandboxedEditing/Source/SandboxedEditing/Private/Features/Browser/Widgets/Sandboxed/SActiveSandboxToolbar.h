// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FActiveSandboxDetailsViewModel;

/** Widget displayed on the top while in an active sandbox. */
class SActiveSandboxToolbar : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SActiveSandboxToolbar) {}
		/** The command list to bind commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<FActiveSandboxDetailsViewModel>& InActiveSandboxViewModel);
	
private:
	
	/** Details about the active sandbox. */
	TSharedPtr<FActiveSandboxDetailsViewModel> ActiveSandboxViewModel;
	
	/** @return Widget displaying the active sandbox name. */
	TSharedRef<SWidget> MakeNameWidget() const;
	
	/** @return Tooltip text to display for the leave button */
	FText GetLeaveButtonToolTipText() const;
};
}

