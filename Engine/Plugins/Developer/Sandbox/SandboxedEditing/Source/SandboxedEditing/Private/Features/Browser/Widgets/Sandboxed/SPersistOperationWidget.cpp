// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPersistOperationWidget.h"

#include "Features/Browser/ViewModels/FileState/FileStateColumnRegistry.h"
#include "Features/Browser/ViewModels/FileState/FileStateItemUtils.h"
#include "Features/Browser/ViewModels/FileState/FilterFileStateViewModel.h"
#include "Features/Browser/ViewModels/FileState/Models/StaticFileStateViewModel.h"
#include "Features/Browser/ViewModels/Persist/PersistOperationViewModel.h"
#include "Features/Browser/Widgets/Shared/FileState/SFilterableFileStateListView.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Types/GatheredFileChanges.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SPersistWidget"

namespace UE::SandboxedEditing
{
void SPersistOperationWidget::Construct(const FArguments& InArgs, const TSharedRef<FPersistOperationViewModel>& InViewModel)
{
	ViewModel = InViewModel;
	
	const FFileStateColumnRegistry Columns = GetColumnsForPersist(ViewModel.ToSharedRef());
	const TSharedRef<FFilterFileStateViewModel> FilterViewModel = MakeShared<FFilterFileStateViewModel>(Columns.ToBehaviorArray());
	const TSharedRef<FStaticFileStateViewModel> FileStateViewModel = MakeShared<FStaticFileStateViewModel>(
		Columns.ColumnBehaviors, 
		MakeItemsFromFileChanges(ViewModel->GetPersistableFiles()),
		FilterViewModel
		);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(5.f)
			[
				SNew(SFilterableFileStateListView, FileStateViewModel, FilterViewModel)
				.ColumnFactories(Columns.ColumnFactories)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(5.f)
			[
				MakeButtonArea()
			]
		]
	];
}

TSharedRef<SWidget> SPersistOperationWidget::MakeButtonArea()
{
	return SNew(SUniformGridPanel)
		.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
		.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
		.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
		+SUniformGridPanel::Slot(0,0)
		[
			SNew(SPositiveActionButton)
			.Text(LOCTEXT("PersistButton", "Persist"))
			.IsEnabled(this, &SPersistOperationWidget::IsPersistButtonEnabled)
			.OnClicked(this, &SPersistOperationWidget::OnClickPersist)
		]
		+SUniformGridPanel::Slot(1,0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.Text(LOCTEXT("CancelButton", "Cancel"))
			.OnClicked(this, &SPersistOperationWidget::OnClickCancel)
		];
}

bool SPersistOperationWidget::IsPersistButtonEnabled() const
{
	return ViewModel->AreAnyFilesMarkedForPersist();
}

FReply SPersistOperationWidget::OnClickPersist() const
{
	ViewModel->ConfirmPersist();
	return FReply::Handled();
}

FReply SPersistOperationWidget::OnClickCancel() const
{
	ViewModel->CancelPersist();
	return FReply::Handled();
}
}

#undef LOCTEXT_NAMESPACE