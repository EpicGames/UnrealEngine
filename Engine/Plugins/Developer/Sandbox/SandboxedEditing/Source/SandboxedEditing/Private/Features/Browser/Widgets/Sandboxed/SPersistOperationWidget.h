// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SandboxedEditing
{
class FStaticFileStateViewModel;
class FPersistOperationViewModel;

/** Displays all UI for persisting a sandbox */
class SPersistOperationWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SPersistOperationWidget){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<FPersistOperationViewModel>& InViewModel);
	
private:
	
	/** Gets the view model being visualized. */
	TSharedPtr<FPersistOperationViewModel> ViewModel;

	bool IsPersistButtonEnabled() const;
	TSharedRef<SWidget> MakeButtonArea();

	FReply OnClickPersist() const;
	FReply OnClickCancel() const;
};
}

