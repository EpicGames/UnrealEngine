// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActionFileStateColumn.h"

#include "SandboxedEditingStyle.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Types/SandboxFileChange.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FActionFileStateColumn"

namespace UE::SandboxedEditing
{
namespace FileActionDetail
{
static const FSlateBrush* GetActionBrush(FileSandboxCore::ESandboxFileChange InChange)
{
	switch (InChange)
	{
	case FileSandboxCore::ESandboxFileChange::Added: return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.OpenForAdd");
	case FileSandboxCore::ESandboxFileChange::Removed: return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.MarkedForDelete");
	case FileSandboxCore::ESandboxFileChange::Edited: return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.CheckedOut");
	case FileSandboxCore::ESandboxFileChange::None: [[fallthrough]];
	default: return nullptr;
	}
}

static FText GetActionToolTipText(FileSandboxCore::ESandboxFileChange InChange)
{
	switch (InChange)
	{
	case FileSandboxCore::ESandboxFileChange::Added: return LOCTEXT("Added", "Marked for add");
	case FileSandboxCore::ESandboxFileChange::Removed: return LOCTEXT("Removed", "Marked for delete");
	case FileSandboxCore::ESandboxFileChange::Edited: return LOCTEXT("Edited", "File is edited");
	case FileSandboxCore::ESandboxFileChange::None: [[fallthrough]];
	default: return FText::GetEmpty();
	}
}
}

void FActionFileStateColumn::PopulateSearchTerms(const TSharedPtr<FFileStateItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	switch (InRowData->Action)
	{
	case FileSandboxCore::ESandboxFileChange::Added:
		OutSearchTerms.Add(TEXT("add"));
		break;
	case FileSandboxCore::ESandboxFileChange::Removed:
		OutSearchTerms.Add(TEXT("remove"));
		OutSearchTerms.Add(TEXT("delete"));
		break;
	case FileSandboxCore::ESandboxFileChange::Edited:
		OutSearchTerms.Add(TEXT("edit"));
		OutSearchTerms.Add(TEXT("modify"));
		break;
	default: break;
	}
}

bool FActionFileStateColumn::SortBy(const TSharedPtr<FFileStateItem>& Lhs, const TSharedPtr<FFileStateItem>& Rhs, EColumnSortMode::Type SortMode) const
{
	// Sort by action type (enum value order)
	return SortMode == EColumnSortMode::Ascending
		? static_cast<uint8>(Lhs->Action) < static_cast<uint8>(Rhs->Action)
		: static_cast<uint8>(Lhs->Action) > static_cast<uint8>(Rhs->Action);
}

SHeaderRow::FColumn::FArguments FActionFileStateColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(FileActionColumn)
		.DefaultLabel(FText::GetEmpty())
		.ToolTipText(LOCTEXT("ToolTip", "The action that was performed on the file"))
		.FixedWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.FileActions.FileActionColumn.FixedWidth"))
		.ShouldGenerateWidget(true)
		[
			SNullWidget::NullWidget
		];
}

TSharedRef<SWidget> FActionFileStateColumn::MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.FileActions.FileActionColumn.Padding"))
		[
			SNew(SImage)
			.Image(FileActionDetail::GetActionBrush(InArgs.RowData->Action))
			.ToolTipText(FileActionDetail::GetActionToolTipText(InArgs.RowData->Action))
		];
}
}

#undef LOCTEXT_NAMESPACE