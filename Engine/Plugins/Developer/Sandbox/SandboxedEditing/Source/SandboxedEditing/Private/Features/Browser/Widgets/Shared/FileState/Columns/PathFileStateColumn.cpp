// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathFileStateColumn.h"

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FPathFileStateColumn"

namespace UE::SandboxedEditing
{
namespace PathFileColumnDetail
{
/** @return A file path to display in the column, kept as short as possible. */
static FString MakeProjectRelative(const FString& InPath)
{
	const FString AbsPath = FPaths::ConvertRelativePathToFull(InPath);
	
	// Most content will be in content folder, so avoid the Content/ prefix...
	FString Result = AbsPath;
	if (FPaths::MakePathRelativeTo(Result, *FPaths::ProjectContentDir()))
	{
		return Result;
	}
	
	// ... if it's plugin content, we'll prefix with /Plugins/x
	Result = AbsPath;
	if (FPaths::MakePathRelativeTo(Result, *FPaths::ProjectDir()))
	{
		return Result;
	}
	
	// And otherwise absolute path, e.g. if editing engine content
	return AbsPath;
}
}

void FPathFileStateColumn::PopulateSearchTerms(const TSharedPtr<FFileStateItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(
		PathFileColumnDetail::MakeProjectRelative(InRowData->NonSandboxFile)
		);
}

bool FPathFileStateColumn::SortBy(const TSharedPtr<FFileStateItem>& Lhs, const TSharedPtr<FFileStateItem>& Rhs, EColumnSortMode::Type SortMode) const
{
	const FString LhsPath = PathFileColumnDetail::MakeProjectRelative(Lhs->NonSandboxFile);
	const FString RhsPath = PathFileColumnDetail::MakeProjectRelative(Rhs->NonSandboxFile);
	return SortMode == EColumnSortMode::Ascending ? LhsPath < RhsPath : LhsPath > RhsPath;
}

SHeaderRow::FColumn::FArguments FPathFileStateColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(FilePathColumn)
		.FillWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.FileActions.PathColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("Name.Label", "Path"))
		.ToolTipText(LOCTEXT("Name.Description", "The absolute file path to the file"))
		.ShouldGenerateWidget(true);
}

TSharedRef<SWidget> FPathFileStateColumn::MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs)
{
	const FText DisplayText = FText::AsCultureInvariant(
		PathFileColumnDetail::MakeProjectRelative(InArgs.RowData->NonSandboxFile)
		);
	const FText AbsPath =  FText::AsCultureInvariant(
		FPaths::ConvertRelativePathToFull(InArgs.RowData->NonSandboxFile)
		);
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.FileActions.PathColumn.Padding"))
		[
			SNew(STextBlock)
			.Text(DisplayText)
			.ToolTipText(AbsPath)
			.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
			.HighlightText(InArgs.HighlightText)
		];
}
}

#undef LOCTEXT_NAMESPACE