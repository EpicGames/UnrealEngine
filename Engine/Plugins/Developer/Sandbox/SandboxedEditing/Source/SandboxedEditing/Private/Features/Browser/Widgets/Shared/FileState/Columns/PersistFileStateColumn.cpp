// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersistFileStateColumn.h"

#include "SandboxedEditingStyle.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Features/Browser/ViewModels/Persist/PersistOperationViewModel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FPersistFileStateColumn"

namespace UE::SandboxedEditing
{
FPersistFileStateColumn::FPersistFileStateColumn(
const TSharedRef<FPersistOperationViewModel>& InPersistViewModel
	)
	: PersistViewModel(InPersistViewModel)
{}

SHeaderRow::FColumn::FArguments FPersistFileStateColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(PersistCheckboxColumn)
		.FixedWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.FileActions.PersistCheckbox.FixedWidth"))
		.ShouldGenerateWidget(true)
		[
			SNew(SCheckBox)
			.IsChecked_Raw(this, &FPersistFileStateColumn::GetToggleRootSelectedState)
			.OnCheckStateChanged_Raw(this, &FPersistFileStateColumn::OnToggleRootSelectedCheckBox)
		];
}

TSharedRef<SWidget> FPersistFileStateColumn::MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs)
{
	const int32 Index = PersistViewModel->GetPersistableFiles().NonSandboxPaths.IndexOfByKey(InArgs.RowData->NonSandboxFile);
	if (Index == INDEX_NONE)
	{
		return SNullWidget::NullWidget;
	}
	
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.FileActions.PersistCheckbox.Padding"))
		[
			SNew(SCheckBox)
			.IsChecked_Raw(this, &FPersistFileStateColumn::GetItemCheckBoxState, Index)
			.OnCheckStateChanged_Raw(this, &FPersistFileStateColumn::SetItemCheckBoxState, Index)
		];
}

ECheckBoxState FPersistFileStateColumn::GetToggleRootSelectedState() const
{
	return PersistViewModel->AreAllFilesMarkedForPersist() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FPersistFileStateColumn::OnToggleRootSelectedCheckBox(ECheckBoxState InNewState) const
{
	PersistViewModel->SetAllFilesPersisted(InNewState == ECheckBoxState::Checked);
}

ECheckBoxState FPersistFileStateColumn::GetItemCheckBoxState(int32 InIndex) const
{
	return PersistViewModel->IsFileMarkedForPersist(InIndex) 
		? ECheckBoxState::Checked 
		: ECheckBoxState::Unchecked;
}

void FPersistFileStateColumn::SetItemCheckBoxState(ECheckBoxState InNewState, int32 InIndex) const
{
	PersistViewModel->SetFilePersisted(InIndex, InNewState == ECheckBoxState::Checked);
}
}

#undef LOCTEXT_NAMESPACE