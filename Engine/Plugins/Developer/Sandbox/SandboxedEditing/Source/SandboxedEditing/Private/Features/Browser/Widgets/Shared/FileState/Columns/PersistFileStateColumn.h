// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/IFileStateColumnBehavior.h"
#include "Features/Browser/ViewModels/FileState/IFileStateColumnWidgetFactory.h"

namespace UE::SandboxedEditing
{
class FPersistOperationViewModel;

class FPersistFileStateColumn : public IFileStateColumnBehavior, public IFileStateColumnWidgetFactory
{
public:
	
	explicit FPersistFileStateColumn(const TSharedRef<FPersistOperationViewModel>& InPersistViewModel);

	//~ Begin IFileStateColumnBehavior Interface
	virtual void PopulateSearchTerms(const TSharedPtr<FFileStateItem>& InRowData, TArray<FString>& OutSearchTerms) const override {}
	virtual bool CanSort() const override { return false; }
	virtual bool SortBy(const TSharedPtr<FFileStateItem>& Lhs, const TSharedPtr<FFileStateItem>& Rhs, EColumnSortMode::Type SortMode) const override { return false; }
	//~ End IFileStateColumnBehavior Interface

	//~ Begin IFileStateWidgetFactory Interface
	virtual SHeaderRow::FColumn::FArguments MakeColumnArguments() override;
	virtual TSharedRef<SWidget> MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs) override;
	//~ End IFileStateWidgetFactory Interface
	
private:
	
	/** Used to set whether an item should be persisted */
	const TSharedRef<FPersistOperationViewModel> PersistViewModel;

	ECheckBoxState GetToggleRootSelectedState() const;
	void OnToggleRootSelectedCheckBox(ECheckBoxState InNewState) const;

	ECheckBoxState GetItemCheckBoxState(int32 InIndex) const;
	void SetItemCheckBoxState(ECheckBoxState InNewState, int32 InIndex) const;
};
}

