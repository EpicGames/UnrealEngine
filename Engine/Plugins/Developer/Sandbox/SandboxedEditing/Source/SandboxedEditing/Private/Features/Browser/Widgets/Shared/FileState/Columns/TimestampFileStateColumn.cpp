// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimestampFileStateColumn.h"

#include "SandboxedEditingStyle.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FTimestampFileStateColumn"

namespace UE::SandboxedEditing
{
namespace TimestampDetail
{
static FText FormatRelativeTime(const FDateTime& EventTime, const FDateTime* CurrTime = nullptr)
{
	FDateTime BaseTime = CurrTime ? *CurrTime : FDateTime::UtcNow();

	FTimespan TimeSpan = BaseTime - EventTime;
	int32 Days = TimeSpan.GetDays();
	int32 Hours = TimeSpan.GetHours();

	if (Days >= 1)
	{
		return Hours > 0 ?
			FText::Format(LOCTEXT("DaysHours", "{0} {0}|plural(one=Day,other=Days), {1} {1}|plural(one=Hour,other=Hours) Ago"), Days, Hours) :
			FText::Format(LOCTEXT("Days", "{0} {0}|plural(one=Day,other=Days) Ago"), Days);
	}

	int32 Minutes = TimeSpan.GetMinutes();
	if (Hours >= 1)
	{
		return Minutes > 0 ?
			FText::Format(LOCTEXT("HoursMins", "{0} {0}|plural(one=Hour,other=Hours), {1} {1}|plural(one=Minute,other=Minutes) Ago"), Hours, Minutes) :
			FText::Format(LOCTEXT("Hours", "{0} {0}|plural(one=Hour,other=Hours) Ago"), Hours);
	}

	int32 Seconds = TimeSpan.GetSeconds();
	if (Minutes >= 1)
	{
		return Seconds > 0 ?
			FText::Format(LOCTEXT("MinsSecs", "{0} {0}|plural(one=Minute,other=Minutes), {1} {1}|plural(one=Second,other=Seconds) Ago"), Minutes, Seconds) :
			FText::Format(LOCTEXT("Mins", "{0} {0}|plural(one=Minute,other=Minutes) Ago"), Minutes, Seconds);
	}

	if (Seconds >= 1)
	{
		return FText::Format(LOCTEXT("Secs", "{0} {0}|plural(one=Second,other=Seconds) Ago"), Seconds);
	}
	return LOCTEXT("Now", "Now");
}

static FText GetRelativeTimestamp(const FFileStateItem& InItem)
{
	return InItem.HasValidTimestamp()
		? FormatRelativeTime(InItem.Timestamp)
		: LOCTEXT("NotApplicable", "n/a");
}
}

void FTimestampFileStateColumn::PopulateSearchTerms(const TSharedPtr<FFileStateItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(
		TimestampDetail::GetRelativeTimestamp(*InRowData).ToString()
		);
}

bool FTimestampFileStateColumn::SortBy(const TSharedPtr<FFileStateItem>& Lhs, const TSharedPtr<FFileStateItem>& Rhs, EColumnSortMode::Type SortMode) const
{
	// Sort by timestamp (most recent first in descending, oldest first in ascending)
	return SortMode == EColumnSortMode::Ascending
		? Lhs->Timestamp < Rhs->Timestamp
		: Lhs->Timestamp > Rhs->Timestamp;
}

SHeaderRow::FColumn::FArguments FTimestampFileStateColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(FileTimestampColumn)
		.FixedWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.FileActions.TimestampColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("Name.Label", "Time"))
		.ToolTipText(LOCTEXT("Name.Description", "Indicates when this action was performed"));
}

TSharedRef<SWidget> FTimestampFileStateColumn::MakeColumnWidget(const FMakeFileStateColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.FileActions.TimestampColumn.Padding"))
		[
			SNew(STextBlock)
			.Text_Lambda([Item = InArgs.RowData] { return TimestampDetail::GetRelativeTimestamp(*Item); })
			.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
			.HighlightText(InArgs.HighlightText)
		];
}
}
#undef LOCTEXT_NAMESPACE
