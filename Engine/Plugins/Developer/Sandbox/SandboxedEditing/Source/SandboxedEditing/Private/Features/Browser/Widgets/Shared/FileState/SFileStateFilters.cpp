// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFileStateFilters.h"

#include "Features/Browser/ViewModels/FileState/FilterFileStateViewModel.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SFileStateFilters"

namespace UE::SandboxedEditing
{
void SFileStateFilters::Construct(const FArguments& InArgs, const TSharedRef<FFilterFileStateViewModel>& InFilterModel)
{
	FilterViewModel = InFilterModel;
	
	ChildSlot
	[
		SNew(SSearchBox)
		.HintText(LOCTEXT("SearchHint", "Search file states"))
		.OnTextChanged(this, &SFileStateFilters::OnSearchTextChanged)
		.OnTextCommitted(this, &SFileStateFilters::OnSearchTextCommitted)
		.DelayChangeNotificationsWhileTyping(true)
	];
}

void SFileStateFilters::OnSearchTextChanged(const FText& Text) const
{
	FilterViewModel->SetSearchText(Text);
}

void SFileStateFilters::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type Arg)
{
	if (!InFilterText.EqualTo(FilterViewModel->GetSearchText()))
	{
		OnSearchTextChanged(InFilterText);
	}
}
}

#undef LOCTEXT_NAMESPACE