// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SandboxedEditing
{
class FFilterFileStateViewModel;

/** Displays the widgets required to filter file actions, i.e. FFilterFileStateViewModel. */
class SFileStateFilters : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFileStateFilters) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<FFilterFileStateViewModel>& InFilterModel);
	
private:
	
	/** Knows how to filter file state items. */
	TSharedPtr<FFilterFileStateViewModel> FilterViewModel;
	
	void OnSearchTextChanged(const FText& Text) const;
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type Arg);
};
}

