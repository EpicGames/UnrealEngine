// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFileStateListRow.h"

#include "Features/Browser/ViewModels/FileState/IFileStateColumnWidgetFactory.h"

namespace UE::SandboxedEditing
{
void SFileStateListRow::Construct(const FArguments& InArgs, const TSharedRef<FFileStateItem>& InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	Item = InItem;
	ColumnFactoryMap = InArgs._ColumnFactories;
	HighlightText = InArgs._HighlightText;
	
	SMultiColumnTableRow::Construct(
		SMultiColumnTableRow::FArguments().Padding(InArgs._Padding),
		InOwnerTable
		);
}

TSharedRef<SWidget> SFileStateListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (TSharedRef<IFileStateColumnWidgetFactory>* Factory = ColumnFactoryMap.Find(InColumnName))
	{
		return Factory->Get().MakeColumnWidget(
			FMakeFileStateColumnWidgetArgs(Item, HighlightText)
		);
	}
	
	checkNoEntry();
	return SNullWidget::NullWidget;
}
}
