// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/FileState/FileStateItem.h"
#include "Features/Browser/ViewModels/FileState/IFileStateColumnWidgetFactory.h"
#include "Features/Browser/ViewModels/List/ISandboxColumnWidgetFactory.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SandboxedEditing
{
class SFileStateListRow : public SMultiColumnTableRow<TSharedPtr<FFileStateItem>>
{
public:
	
	SLATE_BEGIN_ARGS(SFileStateListRow){}
		/** Used to build the columns. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
		/** Padding for the row */
		SLATE_ARGUMENT(FMargin, Padding)
		
		/** The text to highlight in the columns. */
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<FFileStateItem>& InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	//~ Begin SMultiColumnTableRow Interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow Interface
	
private:
	
	/** The item that is being displayed by this row. */
	TSharedPtr<FFileStateItem> Item;
	
	/** Used to build the columns. */
	FFileStateColumnFactoryMap ColumnFactoryMap;
	
	/** The text to highlight in the columns. */
	TAttribute<FText> HighlightText;
};
}

