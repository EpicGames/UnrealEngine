// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFileStateListView.h"

#include "Features/Browser/ViewModels/FileState/IFileStateColumnWidgetFactory.h"
#include "Features/Browser/ViewModels/FileState/IFileStateViewModel.h"
#include "SFileStateListRow.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SFileStateListView"

namespace UE::SandboxedEditing
{
void SFileStateListView::Construct(const FArguments& InArgs, const TSharedRef<IFileStateViewModel>& InViewModel)
{
	ViewModel = InViewModel;
	ColumnFactories = InArgs._ColumnFactories;
	HighlightText = InArgs._HighlightText;
	OverrideOverlayText = InArgs._OverrideOverlayText;
	NoChangesText = InArgs._NoChangesText.IsEmpty() 
		? LOCTEXT("NoItemsDefault", "No file changes have been made in the sandbox.") : InArgs._NoChangesText;
	
	ChildSlot
	[
		SNew(SOverlay)

		+SOverlay::Slot()
		[
			SAssignNew(ListView, SListView<TSharedPtr<FFileStateItem>>)
			.ListItemsSource(&ViewModel->GetItems())
			.OnGenerateRow(this, &SFileStateListView::OnGenerateRow)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow(MakeHeaderRow())
			.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		]

		+SOverlay::Slot()
		.Padding(0, 20, 0, 0) // To ensure the panel is below the row header.
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SFileStateListView::GetOverlayText)
				.Visibility(this, &SFileStateListView::GetOverlayTextVisibility)
			]
		]
	];
	
	ViewModel->OnItemsChanged().AddSP(ListView.ToSharedRef(), &SListView<TSharedPtr<FFileStateItem>>::RequestListRefresh);
}

TArray<TSharedPtr<FFileStateItem>> SFileStateListView::GetSelectedFiles() const
{
	return ListView->GetSelectedItems();
}

TSharedRef<SHeaderRow> SFileStateListView::MakeHeaderRow()
{
	const TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

	for (FName ColumnId : ViewModel->GetDisplayedColumns())
	{
		if (const TSharedRef<IFileStateColumnWidgetFactory>* Factory = ColumnFactories.Find(ColumnId))
		{
			SHeaderRow::FColumn::FArguments ColumnArgs = Factory->Get().MakeColumnArguments();
			ColumnArgs
				.SortMode(ViewModel.ToSharedRef(), &IFileStateViewModel::GetColumnSortMode, ColumnId)
				.SortPriority(ViewModel.ToSharedRef(), &IFileStateViewModel::GetColumnSortPriority, ColumnId)
				.OnSort(ViewModel.ToSharedRef(), &IFileStateViewModel::OnColumnSortModeChanged);
			HeaderRow->AddColumn(ColumnArgs);
		}
	}

	return HeaderRow;
}

TSharedRef<ITableRow> SFileStateListView::OnGenerateRow(TSharedPtr<FFileStateItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	check(InItem);
	const FMargin Padding = FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.FileActions.RowPadding");
	
	return SNew(SFileStateListRow, InItem.ToSharedRef(), InOwnerTable)
		.ColumnFactories(ColumnFactories)
		.Padding(Padding)
		.HighlightText(HighlightText);
}

FText SFileStateListView::GetOverlayText() const
{
	const FText Override = OverrideOverlayText.IsBound() || OverrideOverlayText.IsSet() ? OverrideOverlayText.Get() : FText::GetEmpty();
	if (!Override.IsEmpty())
	{
		return Override;
	}
	
	return ViewModel->GetItems().IsEmpty()
		? NoChangesText
		: FText::GetEmpty();
}

EVisibility SFileStateListView::GetOverlayTextVisibility() const
{
	return GetOverlayText().IsEmpty() ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}
}

#undef LOCTEXT_NAMESPACE