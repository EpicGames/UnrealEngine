// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Framework/Views/ITypedTableView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class STableViewBase;
class ITableRow;
class SHeaderRow;

namespace UE::SandboxedEditing
{
class IFileStateViewModel;

/** Displays a list view FFileStateItem provided by IFileStateViewModel. */
class SFileStateListView : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFileStateListView)
		{}
		/** Used to build the columns. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
		
		/** The text to highlight in the columns. */
		SLATE_ATTRIBUTE(FText, HighlightText)
		
		/** Text to display in the content, e.g. when all items are filtered out. Overrides the text when the attribute returns non-empty. */
		SLATE_ATTRIBUTE(FText, OverrideOverlayText)
		/**
		 * The text to display when there are no changes to display.
		 * Defaults to: "No file changes have been made in the sandbox."
		 */
		SLATE_ARGUMENT(FText, NoChangesText)

		/** Used to generate the right-click context menu. */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IFileStateViewModel>& InViewModel);
	
	/** @return Selected items. */
	TArray<TSharedPtr<FFileStateItem>> GetSelectedFiles() const;
	
private:
	
	/** The view model being displayed */
	TSharedPtr<IFileStateViewModel> ViewModel;
	
	/** The list view displaying the items. */
	TSharedPtr<SListView<TSharedPtr<FFileStateItem>>> ListView;
	
	/** Used to build the columns. */
	TMap<FName, TSharedRef<IFileStateColumnWidgetFactory>> ColumnFactories;
	
	/** The text to highlight in the columns. */
	TAttribute<FText> HighlightText;
	/** Text to display in the content, e.g. when all items are filtered out. Overrides the text when the attribute returns non-empty. */
	TAttribute<FText> OverrideOverlayText;
	/** The text to display when there are no changes to display. */
	FText NoChangesText;
	
	/** @return The header row to display at the top of the table */
	TSharedRef<SHeaderRow> MakeHeaderRow();
	
	/** @return Widget for the items */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FFileStateItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable) const;
	
	/** @return Text to display in the content, e.g. when there's no items. */
	FText GetOverlayText() const;
	/** @return The visibility of the overlay text */
	EVisibility GetOverlayTextVisibility() const;
};
}