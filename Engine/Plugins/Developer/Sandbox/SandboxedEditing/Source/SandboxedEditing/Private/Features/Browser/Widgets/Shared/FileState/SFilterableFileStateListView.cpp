// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilterableFileStateListView.h"

#include "SFileStateFilters.h"
#include "Components/VerticalBox.h"
#include "Features/Browser/ViewModels/FileState/FilterFileStateViewModel.h"
#include "Features/Browser/ViewModels/FileState/IFileStateViewModel.h"
#include "Features/Browser/Widgets/Shared/FileState/SFileStateListView.h"

#define LOCTEXT_NAMESPACE "SFilterableFileStateListView"

namespace UE::SandboxedEditing
{
void SFilterableFileStateListView::Construct(
	const FArguments& InArgs, 
	const TSharedRef<IFileStateViewModel>& InFileActionsViewModel, 
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
	)
{
	FileActionViewModel = InFileActionsViewModel;
	FilterViewModel = InFilterViewModel;
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1.f)
		[
			SNew(SFileStateFilters, InFilterViewModel)
		]

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(1.f)
		[
			SNew(SBorder)
			[
				SAssignNew(FileStateWidget, SFileStateListView, InFileActionsViewModel)
				.ColumnFactories(InArgs._ColumnFactories)
				.HighlightText_Lambda([InFilterViewModel]{ return InFilterViewModel->GetSearchText(); })
				.OverrideOverlayText(this, &SFilterableFileStateListView::GetOverrideOverlayText)
				.NoChangesText(InArgs._NoChangesText)
				.OnContextMenuOpening(InArgs._OnContextMenuOpening)
			]
		]
	];
}

FText SFilterableFileStateListView::GetOverrideOverlayText() const
{
	const bool bAllFilteredOut = FilterViewModel->AreAnyFiltersActive() && FileActionViewModel->GetItems().IsEmpty();
	return bAllFilteredOut 
		? LOCTEXT("AllFiltered", "All file actions are filtered.")
		: FText::GetEmpty();
}
}

#undef LOCTEXT_NAMESPACE