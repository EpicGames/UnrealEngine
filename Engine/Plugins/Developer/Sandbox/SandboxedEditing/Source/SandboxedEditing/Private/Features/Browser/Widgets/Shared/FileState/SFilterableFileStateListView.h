// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

namespace UE::SandboxedEditing
{
class FFilterFileStateViewModel;
class IFileStateViewModel;
class SFileStateListView;

/** Vertical arrangement of a filter section (search bar) and SFileStateListView. */
class SFilterableFileStateListView : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SFilterableFileStateListView)
		{}
		/** Used to build the columns. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
		
		/** The text to display when there are no changes to display. */
		SLATE_ARGUMENT(FText, NoChangesText)
		
		/** 
		 * Used to generate the right-click context menu. 
		 * @see GetListView
		 */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, 
		const TSharedRef<IFileStateViewModel>& InFileActionsViewModel, 
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
		);
	
	/** @return List view displaying the items */
	const TSharedPtr<SFileStateListView>& GetListView() const { return FileStateWidget; }
	
private:
	
	/** Used to generate the override overlay text. */
	TSharedPtr<IFileStateViewModel> FileActionViewModel;
	/** Used to generate the override overlay text. */
	TSharedPtr<FFilterFileStateViewModel> FilterViewModel;
	
	/** Displays the items. */
	TSharedPtr<SFileStateListView> FileStateWidget;
	
	/** @return Text indicating that all items are filtered out, or empty. */
	FText GetOverrideOverlayText() const;
};
}

