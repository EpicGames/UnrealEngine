// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxDescription.h"

#include "Features/Browser/ViewModels/SandboxMetaDataViewModel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSandboxDescription"

namespace UE::SandboxedEditing
{
void SSandboxDescription::Construct(const FArguments& InArgs, const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel)
{
	MetaDataViewModel = InMetaDataViewModel;
	SandboxPathAttr = InArgs._SandboxPath;
	
	ChildSlot
	[
		SAssignNew(DescriptionTextCtrl, SMultiLineEditableTextBox)
		.SelectAllTextWhenFocused(true)
		.ModiferKeyForNewLine(EModifierKey::Shift)
		.AutoWrapText(true)
		.RevertTextOnEscape(true)
		.HintText(LOCTEXT("Description.Hint", "Enter a description..."))
		.Text(this, &SSandboxDescription::GetDescription)
		.OnTextCommitted(this, &SSandboxDescription::OnDescriptionCommitted)
	];
}

FText SSandboxDescription::GetDescription() const
{
	const TOptional<FString> SandboxPath = SandboxPathAttr.Get();
	if (!SandboxPath)
	{
		return FText::GetEmpty();
	}
	
	const TOptional<FString> Description = MetaDataViewModel->GetDescription(*SandboxPath);
	return Description ? FText::AsCultureInvariant(*Description) : FText::GetEmpty();
}

void SSandboxDescription::OnDescriptionCommitted(const FText& InNewDescription, ETextCommit::Type InType) const
{
	const TOptional<FString> SandboxPath = SandboxPathAttr.Get();
	if (!SandboxPath)
	{
		return;
	}
	
	const FString NewDescription = InNewDescription.ToString();
	const TOptional<FString> CurrentDescription = MetaDataViewModel->GetDescription(*SandboxPath);
	if (!CurrentDescription || NewDescription == *CurrentDescription)
	{
		return;
	}
	
	MetaDataViewModel->SetDescription(*SandboxPath, InNewDescription.ToString());
	
	// Showing a notification gives feedback to the user that they caused something to happen. 
	// Commit happens e.g. when you click outside the text edit box but would not "feel" like it actualy did anything.
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Info(LOCTEXT("DescriptionEdited", "Description updated"));
	Info.bFireAndForget = true;
	Info.ExpireDuration = 4.f;
	const TSharedPtr<SNotificationItem> Notification = NotificationManager.AddNotification(Info);
	Notification->SetCompletionState(SNotificationItem::CS_Success);
}
}

#undef LOCTEXT_NAMESPACE