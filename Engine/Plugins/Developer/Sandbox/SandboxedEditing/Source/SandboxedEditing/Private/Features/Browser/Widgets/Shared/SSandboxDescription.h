// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SExpandableArea;
class SMultiLineEditableTextBox;

namespace UE::SandboxedEditing
{
class FSandboxMetaDataViewModel;

/** Allows editing a description. The description is displayed in an expandable area to save space. */
class SSandboxDescription : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSandboxDescription) {}
		/** Path to the sandbox for which the description is being edited. Returning unset disables the edit box. */
		SLATE_ATTRIBUTE(TOptional<FString>, SandboxPath)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel);
	
private:
	
	/** Used to edit the description. */
	TSharedPtr<FSandboxMetaDataViewModel> MetaDataViewModel;
	
	/** Path to the sandbox for which the description is being edited. */
	TAttribute<TOptional<FString>> SandboxPathAttr;
	
	/** Contains the description text */
	TSharedPtr<SMultiLineEditableTextBox> DescriptionTextCtrl;
	
	FText GetDescription() const;
	void OnDescriptionCommitted(const FText& InNewDescription, ETextCommit::Type InType) const;
};
}

