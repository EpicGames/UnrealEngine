// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSessionBrowserFrame.h"

#include "Styling/AppStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSessionBrowserFrame"

namespace UE::SandboxedEditing
{
void SSessionBrowserFrame::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(1.0f, 2.0f))
		[
			CreateContent(InArgs)
		]
	];
}

TSharedRef<SVerticalBox> SSessionBrowserFrame::CreateContent(const FArguments& InArgs)
{
	const TSharedRef<SVerticalBox> Content = SNew(SVerticalBox);
	
	AppendControlWidget(InArgs, Content);
	AppendSearchWidget(InArgs, Content);
	
	const TSharedRef<SSplitter> Splitter = SNew(SSplitter)
		.Orientation(Orient_Vertical)
		.MinimumSlotHeight(80.0f) // Prevent widgets from overlapping.
		
		+SSplitter::Slot()
		.Value(0.6)
		[
			InArgs._SessionContent.Widget
		];
			
	
	checkf(InArgs._SessionContent.Widget != SNullWidget::NullWidget, TEXT("This is a required argument."));
	Content->AddSlot()
	.FillHeight(1.0f)
	.Padding(1.0f, 2.0f)
	[
		Splitter
	];
	
	AppendDetailsWidget(InArgs, Splitter);
	
	return Content;
}

void SSessionBrowserFrame::AppendControlWidget(const FArguments& InArgs, const TSharedRef<SVerticalBox>& Content)
{
	const bool bHasControlWidget = InArgs._ControlContent.Widget != SNullWidget::NullWidget; 
	if (bHasControlWidget)
	{
		Content->AddSlot()
		.AutoHeight()
		[
			InArgs._ControlContent.Widget
		];

		Content->AddSlot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorThickness)
			.SeparatorImage(&FCoreStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar").SeparatorBrush)
		];
	}
}

void SSessionBrowserFrame::AppendSearchWidget(const FArguments& InArgs, const TSharedRef<SVerticalBox>& Content)
{
	const bool bHasSearchWidget = InArgs._SearchContent.Widget != SNullWidget::NullWidget; 
	if (bHasSearchWidget)
	{
		Content->AddSlot()
		.AutoHeight()
		.Padding(0.f, 4.f, 0.f, 4.f)
		[
			InArgs._SearchContent.Widget
		];
	}
}
	
void SSessionBrowserFrame::AppendDetailsWidget(const FArguments& InArgs, const TSharedRef<SSplitter>& Content)
{
	const bool bHasDetailsWidget = InArgs._DetailsContent.Widget != SNullWidget::NullWidget;
	if (bHasDetailsWidget)
	{
		Content->AddSlot()
		.Value(0.4)
		[
			SNew(SBox)
			.Padding(FMargin(2.0f, 0.0f))
			[
				InArgs._DetailsContent.Widget
			]
		];
	}
}
}

#undef LOCTEXT_NAMESPACE