// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SSplitter;
class SVerticalBox;

namespace UE::SandboxedEditing
{
/** 
 * Generic widget that lays out widgets related for session management.
 * The concept of a session is kept abstract and depends on the code that uses this widget.
 * 
 * For now only used for Sandboxed Editing. In the future, it should be used across Multi-User, Multi-User Server, and Sandboxed editing.
 * The purpose is to have a central place that organizes the layout of the specific widgets, and avoid code duplication.
 * 
 * The follow elements are laid out:
 * - Controls: buttons to control the sessions
 * - Search box: filtering the sessions
 * - Tree / List view: displays the sessions
 * - Extra info: displays more info about the session
 */
class SSessionBrowserFrame : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowserFrame){}
		SLATE_NAMED_SLOT(FArguments, ControlContent)
		SLATE_NAMED_SLOT(FArguments, SearchContent)
		SLATE_NAMED_SLOT(FArguments, SessionContent)
		SLATE_NAMED_SLOT(FArguments, DetailsContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	/** Creates the layout */
	TSharedRef<SVerticalBox> CreateContent(const FArguments& InArgs);
	
	/** Optionally adds the controls (above search), if specified by args. */
	void AppendControlWidget(const FArguments& InArgs, const TSharedRef<SVerticalBox>& Content);
	/** Optionally adds the search widget (above session content), if specified by args. */
	void AppendSearchWidget(const FArguments& InArgs, const TSharedRef<SVerticalBox>& Content);
	/** Optionally adds the controls (below session content), if specified by args. */
	void AppendDetailsWidget(const FArguments& InArgs, const TSharedRef<SSplitter>& Content);
};
}

