// Copyright Epic Games, Inc. All Rights Reserved.

#include "DescriptionSandboxColumn.h"

#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"
#include "Framework/Models/SandboxInfo.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "FDescriptionSandboxColumn"

namespace UE::SandboxedEditing
{
namespace DescriptionDetail
{
static FText FormatDescription(const FSandboxInfo& InRowData)
{
	return FText::AsCultureInvariant(InRowData.Description);
}
}

void FDescriptionSandboxColumn::PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(
		DescriptionDetail::FormatDescription(InRowData->SandboxInfo).ToString()
		);
}

SHeaderRow::FColumn::FArguments FDescriptionSandboxColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(DescriptionSandboxColumn)
		.FillWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.Browser.DescriptionColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("Description.Label", "Description"))
		.ToolTipText(LOCTEXT("Description.ToolTip", "The sandbox description"));
}

TSharedRef<SWidget> FDescriptionSandboxColumn::MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.Browser.DescriptionColumn.Padding"))
		[
			SNew(STextBlock)
			.Text_Lambda([RowData = InArgs.RowData](){ return DescriptionDetail::FormatDescription(RowData->SandboxInfo); })
			.HighlightText(InArgs.HighlightText)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		];
}
}

#undef LOCTEXT_NAMESPACE
