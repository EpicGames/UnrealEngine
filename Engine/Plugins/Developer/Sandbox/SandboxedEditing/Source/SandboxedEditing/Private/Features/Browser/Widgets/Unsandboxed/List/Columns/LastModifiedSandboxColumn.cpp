// Copyright Epic Games, Inc. All Rights Reserved.

#include "LastModifiedSandboxColumn.h"

#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"
#include "Framework/Models/SandboxInfo.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "FLastModifiedSandboxColumn"

namespace UE::SandboxedEditing
{
namespace LastModifiedDetail
{
static FText FormatLastModified(const FSandboxInfo& InRowData)
{
	// InRowData.LastModified is already in local time, hence must pass in FText::GetInvariantTimeZone() to avoid double conversion.
	return FText::AsDateTime(InRowData.LastModified, EDateTimeStyle::Default, EDateTimeStyle::Default, FText::GetInvariantTimeZone());
}
}
void FLastModifiedSandboxColumn::PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(
		LastModifiedDetail::FormatLastModified(InRowData->SandboxInfo).ToString()
		);
}

SHeaderRow::FColumn::FArguments FLastModifiedSandboxColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(LastModifiedSandboxColumn)
		.FillWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.Browser.LastModifiedColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("LastModified.Label", "Last Modified"))
		.ToolTipText(LOCTEXT("LastModified.Description", "When this sandbox was last modified"));
}

TSharedRef<SWidget> FLastModifiedSandboxColumn::MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.Browser.LastModifiedColumn.Padding"))
		[
			SNew(STextBlock)
			.Text_Lambda([RowData = InArgs.RowData](){ return LastModifiedDetail::FormatLastModified(RowData->SandboxInfo); })
			.HighlightText(InArgs.HighlightText)
		];
}
}

#undef LOCTEXT_NAMESPACE