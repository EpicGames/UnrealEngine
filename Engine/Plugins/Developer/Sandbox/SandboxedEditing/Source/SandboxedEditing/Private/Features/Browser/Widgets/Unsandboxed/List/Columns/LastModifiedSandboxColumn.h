// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/List/ISandboxColumnWidgetFactory.h"
#include "Features/Browser/ViewModels/List/ISandboxColumnBehavior.h"

namespace UE::SandboxedEditing
{
/** Displays when the sandbox was last modified. */
class FLastModifiedSandboxColumn : public ISandboxColumnBehavior, public ISandboxColumnWidgetFactory
{
public:
	
	//~ Begin ISandboxColumnBehavior Interface
	virtual void PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const override;
	//~ End ISandboxColumnBehavior Interface
	
	//~ Begin ISandboxWidgetFactory Interface
	virtual SHeaderRow::FColumn::FArguments MakeColumnArguments() override;
	virtual TSharedRef<SWidget> MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs) override;
	//~ End ISandboxWidgetFactory Interface
};
}

