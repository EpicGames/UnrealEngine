// Copyright Epic Games, Inc. All Rights Reserved.

#include "NameSandboxColumn.h"

#include "Containers/Ticker.h"
#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Framework/Models/SandboxInfo.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "FNameSandboxColumn"

namespace UE::SandboxedEditing
{
class SSandboxNameWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSandboxNameWidget){}
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, 
		const TSharedRef<FSandboxControlsViewModel>& InRenameViewModel,
		const FMakeSandboxColumnWidgetArgs& InColumnsArgs
		)
	{
		RenameViewModel = InRenameViewModel;
		Item = InColumnsArgs.RowData;
		
		ChildSlot
		[
			SAssignNew(EditWidget, SInlineEditableTextBlock)
			.Text_Lambda([this] { return FText::AsCultureInvariant(Item->SandboxInfo.Name); })
			.ToolTipText_Lambda([this]
			{
				const FString& Description = Item->SandboxInfo.Description;
				return !Description.IsEmpty()
					? FText::AsCultureInvariant(Description) 
					: LOCTEXT("NoDescription", "This sandbox has no user-provided description");
			})
			.HintText(LOCTEXT("NameHint", "Enter name"))
			.HighlightText(InColumnsArgs.HighlightText)
			.OnTextCommitted(this, &SSandboxNameWidget::OnNameCommitted)
			.IsReadOnly(false)
			.IsSelected(InColumnsArgs.IsSelectedDelegate)
			.OnVerifyTextChanged(this, &SSandboxNameWidget::OnValidateName)
			.OnBeginTextEdit(this, &SSandboxNameWidget::OnBeginTextEdit)
			.OnExitEditingMode(this, &SSandboxNameWidget::OnExitTextEdit)
		];
		
		RenameViewModel->OnRenameWorkflowStarted().AddSP(this, &SSandboxNameWidget::OnRenameWorkflowStarted);
	}
	
private:
	
	/** Informs us when a rename operation is started. */
	TSharedPtr<FSandboxControlsViewModel> RenameViewModel;
	
	/** The item this widget is displaying. */
	TSharedPtr<FSandboxListItem> Item;
	
	/** The widget that does the editing. Used to make it appear like an editable text field. */
	TSharedPtr<SInlineEditableTextBlock> EditWidget;
	
	bool bIsExplicitlyStartingEditOp = false;
	
	void OnRenameWorkflowStarted(FRenameSandboxWorkflow& InWorkflow)
	{
		const bool bPathSameAsItem = FPaths::IsSamePath(InWorkflow.GetRenamedSandboxRoot(), Item->SandboxInfo.SandboxRoot);
		if (bPathSameAsItem
			&& !EditWidget->IsInEditMode()
			&& !bIsExplicitlyStartingEditOp)
		{
			TGuardValue Guard(bIsExplicitlyStartingEditOp, true);
			EditWidget->EnterEditingMode();
		}
	}
	
	void OnBeginTextEdit(const FText& Text)
	{
		if (!bIsExplicitlyStartingEditOp)
		{
			TGuardValue Guard(bIsExplicitlyStartingEditOp, true);
			RenameViewModel->StartRenameWorkflow(Item->SandboxInfo.SandboxRoot);
		}
	}
	
	void OnExitTextEdit()
	{
		// OnExitTextEdit is executed just before OnNameCommitted. In that case, we don't want to cancel the operation.
		// OnExitTextEdit is also executed when editing stops for any other reason. In that case, we do want to cancel the operation.
		ExecuteOnGameThread(TEXT("SSandboxNameWidget"), [WeakModel = RenameViewModel.ToWeakPtr()]
		{
			const TSharedPtr<FSandboxControlsViewModel> ModelPin = WeakModel.Pin();
			FRenameSandboxWorkflow* Workflow = ModelPin ? ModelPin->GetCurrentRenameWorkflow() : nullptr;
			if (Workflow)
			{
				Workflow->Cancel();
			}
		});
	}
	
	void OnNameCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
	{
		FRenameSandboxWorkflow* Workflow = RenameViewModel->GetCurrentRenameWorkflow();
		if (!Workflow)
		{
			return;
		}

		Workflow->SetName(InNewText.ToString());
		switch (InCommitType)
		{
		case ETextCommit::Default: [[fallthrough]];
		case ETextCommit::OnEnter:
		case ETextCommit::OnUserMovedFocus:
			
			Workflow->Confirm();
			break;
		case ETextCommit::OnCleared:
			Workflow->Cancel();
			break;
		}
	}
	
	bool OnValidateName(const FText& InNewText, FText& OutError) const
	{
		if (FRenameSandboxWorkflow* Workflow = RenameViewModel->GetCurrentRenameWorkflow())
		{
			Workflow->SetName(InNewText.ToString());
			return Workflow->IsNameValid(&OutError);
		}
		
		return false;
	}
};

FNameSandboxColumn::FNameSandboxColumn(const TSharedRef<FSandboxControlsViewModel>& InRenameViewModel)
	: RenameViewModel(InRenameViewModel)
{}

void FNameSandboxColumn::PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(InRowData->SandboxInfo.Name);
}

SHeaderRow::FColumn::FArguments FNameSandboxColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(NameSandboxColumn)
		.FillWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.Browser.NameColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("Name.Label", "Sandbox"))
		.ToolTipText(LOCTEXT("Name.Description", "The name of this sandbox"))
		.ShouldGenerateWidget(true);
}

TSharedRef<SWidget> FNameSandboxColumn::MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.Browser.NameColumn.Padding"))
		[
			SNew(SSandboxNameWidget, RenameViewModel, InArgs)
		];
}
}

#undef LOCTEXT_NAMESPACE