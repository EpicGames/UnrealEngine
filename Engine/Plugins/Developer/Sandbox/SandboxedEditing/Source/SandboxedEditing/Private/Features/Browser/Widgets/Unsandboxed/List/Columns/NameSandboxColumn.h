// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/List/ISandboxColumnBehavior.h"
#include "Features/Browser/ViewModels/List/ISandboxColumnWidgetFactory.h"

namespace UE::SandboxedEditing
{
class FSandboxControlsViewModel;

/** Displays the sandbox name. */
class FNameSandboxColumn : public ISandboxColumnBehavior, public ISandboxColumnWidgetFactory
{
public:
	
	explicit FNameSandboxColumn(const TSharedRef<FSandboxControlsViewModel>& InRenameViewModel);
	
	//~ Begin ISandboxColumnBehavior Interface
	virtual void PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const override;
	//~ End ISandboxColumnBehavior Interface
	
	//~ Begin ISandboxWidgetFactory Interface
	virtual SHeaderRow::FColumn::FArguments MakeColumnArguments() override;
	virtual TSharedRef<SWidget> MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs) override;
	//~ End ISandboxWidgetFactory Interface
	
private:
	
	/** View model used to rename sandboxes. */
	const TSharedRef<FSandboxControlsViewModel> RenameViewModel;
};
}

