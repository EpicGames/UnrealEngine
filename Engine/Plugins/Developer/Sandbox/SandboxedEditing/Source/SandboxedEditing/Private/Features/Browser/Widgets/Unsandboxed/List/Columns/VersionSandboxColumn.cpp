// Copyright Epic Games, Inc. All Rights Reserved.

#include "VersionSandboxColumn.h"

#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"
#include "Framework/Models/SandboxInfo.h"
#include "SandboxedEditingStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "FVersionSandboxColumn"

namespace UE::SandboxedEditing
{
namespace VersionDetail
{
static FText FormatVersion(const FSandboxInfo& InRowData)
{
	return FText::AsCultureInvariant(InRowData.EngineVersion.ToString());
}
}
void FVersionSandboxColumn::PopulateSearchTerms(const TSharedPtr<FSandboxListItem>& InRowData, TArray<FString>& OutSearchTerms) const
{
	OutSearchTerms.Add(
		VersionDetail::FormatVersion(InRowData->SandboxInfo).ToString()
		);
}

SHeaderRow::FColumn::FArguments FVersionSandboxColumn::MakeColumnArguments()
{
	return SHeaderRow::FColumn::FArguments()
		.ColumnId(VersionSandboxNameColumn)
		.FillWidth(FSandboxedEditingStyle::Get().GetFloat("SandboxedEditing.Browser.VersionColumn.FillWidth"))
		.DefaultLabel(LOCTEXT("Version.Label", "Engine Version"))
		.ToolTipText(LOCTEXT("Version.Description", "The engine version this sandbox had when it was created."));
}

TSharedRef<SWidget> FVersionSandboxColumn::MakeColumnWidget(const FMakeSandboxColumnWidgetArgs& InArgs)
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.Browser.VersionColumn.Padding"))
		[
			SNew(STextBlock)
			.Text_Lambda([RowData = InArgs.RowData](){ return VersionDetail::FormatVersion(RowData->SandboxInfo); })
			.HighlightText(InArgs.HighlightText)
		];
}
}

#undef LOCTEXT_NAMESPACE