// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "HAL/Platform.h"

namespace UE::SandboxedEditing
{
class FSandboxListItem;

/** The intent of the dummy sandbox item. */
enum class EDummySandboxItemType : uint8
{
	CreateSandbox
};

/** 
 * Holds info about a FSandboxListItem added to the SListView but does not really represent a real sandbox. 
 * It's a way to get the SListView to display row widget.
 */
struct FDummySandboxItemInfo
{
	const TSharedRef<FSandboxListItem> Item;
	const EDummySandboxItemType Type;

	explicit FDummySandboxItemInfo(const TSharedRef<FSandboxListItem>& InItem, EDummySandboxItemType InType)
		: Item(InItem)
		, Type(InType)
	{}
};
}