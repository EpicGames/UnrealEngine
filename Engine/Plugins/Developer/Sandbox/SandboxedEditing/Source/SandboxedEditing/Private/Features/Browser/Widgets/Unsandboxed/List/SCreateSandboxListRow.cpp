// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCreateSandboxListRow.h"

#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/SandboxCreationWorkflow.h"
#include "Framework/Application/SlateApplication.h"
#include "Utils/WidgetUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "SCreateSandboxListRow"

namespace UE::SandboxedEditing
{
void SCreateSandboxListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
{
	WorkflowAttr = InArgs._Workflow;
	
	SMultiColumnTableRow::Construct(
		SMultiColumnTableRow::FArguments().Padding(InArgs._Padding),
		InOwnerTable
		);
}

void SCreateSandboxListRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SMultiColumnTableRow<TSharedPtr<FSandboxListItem>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (NameWidget && !bHasTakenInitialFocus)
	{
		bHasTakenInitialFocus = FSlateApplication::Get().SetKeyboardFocus(NameWidget.ToSharedRef());
	}
}

TSharedRef<SWidget> SCreateSandboxListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (InColumnName == NameSandboxColumn)
	{
		return SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(1.f, 1.f, 4.f, 1.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
			
			+SHorizontalBox::Slot()
			[
				CreateNameWidget()
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			[
				CreateButtons()
			];
	}
	
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SCreateSandboxListRow::CreateNameWidget()
{
	return SAssignNew(NameWidget, SEditableTextBox)
		.HintText(LOCTEXT("EnterSessionNameHint", "Enter sandbox name"))
		.OnTextCommitted(this, &SCreateSandboxListRow::OnSandboxNameCommitted)
		.OnKeyDownHandler(this, &SCreateSandboxListRow::OnKeyDownHandler)
		.OnTextChanged(this, &SCreateSandboxListRow::OnSandboxNameChanged);
}

TSharedRef<SWidget> SCreateSandboxListRow::CreateButtons()
{
	return SNew(SUniformGridPanel)
		.SlotPadding(FMargin(1.0f, 0.0f))

		// 'Accept' button
		+SUniformGridPanel::Slot(0, 0)
		[
			MakePositiveActionButton(
				FAppStyle::GetBrush("Icons.Check"),
				LOCTEXT("CreateCheckIconTooltip", "Create the session"),
				TAttribute<bool>::Create([this]
				{
					const FSandboxCreationWorkflow* Workflow = GetWorkflow();
					return Workflow && Workflow->CanConfirm();
				}),
				FOnClicked::CreateLambda([this]
				{
					if (FSandboxCreationWorkflow* Workflow = GetWorkflow())
					{
						Workflow->Confirm();
					}
					return FReply::Handled();
				}))
		]

		// 'Decline' button
		+SUniformGridPanel::Slot(1, 0)
		[
			MakeNegativeActionButton(
				FAppStyle::GetBrush("Icons.X"),
				LOCTEXT("CancelIconTooltip", "Cancel"),
				true, // Always enabled.
				FOnClicked::CreateLambda([this]
				{
					if (FSandboxCreationWorkflow* Workflow = GetWorkflow())
					{
						Workflow->Cancel();
					}
					return FReply::Handled();
				}))
		];
}

void SCreateSandboxListRow::OnSandboxNameCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	FSandboxCreationWorkflow* Workflow = GetWorkflow();
	if (!Workflow)
	{
		return;
	}
	
	Workflow->SetName(InNewText.ToString());
	if (Workflow->CanConfirm() && InCommitType == ETextCommit::OnEnter)
	{
		Workflow->Confirm();
	}
}

FReply SCreateSandboxListRow::OnKeyDownHandler(const FGeometry&, const FKeyEvent& InKeyEvent)
{
	FSandboxCreationWorkflow* Workflow = GetWorkflow();
	if (InKeyEvent.GetKey() == EKeys::Escape && Workflow)
	{
		Workflow->Cancel();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SCreateSandboxListRow::OnSandboxNameChanged(const FText& InNewText)
{
	if (FSandboxCreationWorkflow* Workflow = GetWorkflow())
	{
		const FString Name = InNewText.ToString();
		Workflow->SetName(Name);
		
		FText Error = FText::GetEmpty();
		Workflow->IsValidName(Name, &Error);
		NameWidget->SetError(Error);
	}
}
}

#undef LOCTEXT_NAMESPACE