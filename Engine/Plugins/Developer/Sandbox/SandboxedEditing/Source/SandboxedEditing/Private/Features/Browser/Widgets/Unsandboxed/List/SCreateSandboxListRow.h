// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SEditableTextBox;

namespace UE::SandboxedEditing
{
class FSandboxCreationWorkflow;
class FSandboxListItem;

/** A row in the SSandboxListView for creating a new sandbox. */
class SCreateSandboxListRow : public SMultiColumnTableRow<TSharedPtr<FSandboxListItem>>
{
public:
	
	SLATE_BEGIN_ARGS(SCreateSandboxListRow) {}
		/** Gets the creation workflow this widget displays. */
		SLATE_ATTRIBUTE(FSandboxCreationWorkflow*, Workflow)
		
		/** Padding for the row */
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable);
	
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//~ Begin  SMultiColumnTableRow Interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End  SMultiColumnTableRow Interface
	
private:
	
	/** Gets the creation workflow this widget displays. */
	TAttribute<FSandboxCreationWorkflow*> WorkflowAttr;
	
	/** Name widget */
	TSharedPtr<SEditableTextBox> NameWidget;
	
	/** Whether keyboard focus has been set to the editable text already. */
	bool bHasTakenInitialFocus = false;
	
	/** @return The widget for editing the name. */
	TSharedRef<SWidget> CreateNameWidget();
	/** @return Confirm and cancel buttons. */
	TSharedRef<SWidget> CreateButtons();
	
	/** Creates the sandbox if enter is pressed. */
	void OnSandboxNameCommitted(const FText& InNewText, ETextCommit::Type InCommitType);
	/** Handles cancelling when esc is pressed. */
	FReply OnKeyDownHandler(const FGeometry& Geometry, const FKeyEvent& InKeyEvent);
	/** Handles displaying an error. */
	void OnSandboxNameChanged(const FText& InNewText);
	
	/** @return Gets the workflow being displayed. */
	FSandboxCreationWorkflow* GetWorkflow() const { return WorkflowAttr.Get(); }
};
}

