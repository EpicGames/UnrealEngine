// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxListRow.h"

#include "Features/Browser/ViewModels/List/ISandboxColumnWidgetFactory.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"

namespace UE::SandboxedEditing
{
void SSandboxListRow::Construct(const FArguments& InArgs, const TSharedRef<FSandboxListItem>& InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	Item = InItem;
	ColumnFactoryMap = InArgs._ColumnFactories;
	
	HighlightTextAttr = InArgs._HighlightText;
	IsSelectedAttr = InArgs._IsSelected;
	
	SMultiColumnTableRow::Construct(
		SMultiColumnTableRow::FArguments().Padding(InArgs._Padding),
		InOwnerTable
		);
}

TSharedRef<SWidget> SSandboxListRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (TSharedRef<ISandboxColumnWidgetFactory>* Factory = ColumnFactoryMap.Find(InColumnName))
	{
		return Factory->Get().MakeColumnWidget(
			FMakeSandboxColumnWidgetArgs(
				Item.ToSharedRef(), HighlightTextAttr,
				FIsSelected::CreateLambda([this]{ return IsSelectedAttr.Get(); })
			)
		);
	}

	checkNoEntry();
	return SNullWidget::NullWidget;
}
}
