// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SandboxedEditing
{
class FSandboxListItem;

/** A row in the SSandboxListView displaying a single sandbox. */
class SSandboxListRow : public SMultiColumnTableRow<TSharedPtr<FSandboxListItem>>
{
public:
	
	SLATE_BEGIN_ARGS(SSandboxListRow) {}
		/** The columns we know about */
		SLATE_ARGUMENT(FSandboxColumnFactoryMap, ColumnFactories)
		
		/** Which text should be highlighted (text search). */
		SLATE_ATTRIBUTE(FText, HighlightText)
		/** Whether the item is selected. */
		SLATE_ATTRIBUTE(bool, IsSelected)

		/** Padding for the row */
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSandboxListItem>& InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	//~ Begin  SMultiColumnTableRow Interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End  SMultiColumnTableRow Interface
	
private:
	
	/** Used to build the columns. */
	FSandboxColumnFactoryMap ColumnFactoryMap;
	
	/** The displayed item. */
	TSharedPtr<FSandboxListItem> Item;
	
	/** Which text should be highlighted (text search). */
	TAttribute<FText> HighlightTextAttr;
	/** Whether the item is selected. */
	TAttribute<bool> IsSelectedAttr;
};
}

