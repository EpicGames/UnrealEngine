// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxListView.h"

#include "Features/Browser/Commands/BrowserCommands.h"
#include "Features/Browser/ViewModels/List/ISandboxColumnBehavior.h"
#include "Features/Browser/ViewModels/List/ISandboxColumnWidgetFactory.h"
#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/List/SandboxListItem.h"
#include "Features/Browser/ViewModels/List/SandboxListViewModel.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Features/Browser/Widgets/Unsandboxed/List/SCreateSandboxListRow.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SSandboxListRow.h"
#include "SandboxedEditingStyle.h"
#include "Utils/BreakBehavior.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SSandboxListView"

namespace UE::SandboxedEditing
{
void SSandboxListView::Construct(const FArguments& InArgs, const FViewModels& InViewModels)
{
	ListViewModel = InViewModels.ListViewModel;
	ControlsViewModel = InViewModels.ControlsViewModel;
	HighlightTextAttr = InArgs._HighlightText;
	ColumnFactoryMap = InArgs._ColumnFactories;
	ContextMenuCommandList = InArgs._ContextMenuCommandList;
	OnSelectedSandboxChangedDelegate = InArgs._OnSelectedSandboxChanged;
	
	ChildSlot
	[
		SNew(SOverlay)
		
		+SOverlay::Slot()
		[
			SAssignNew(ListView, SListView<TSharedPtr<FSandboxListItem>>)
			.ListItemsSource(&Items)
			.SelectionMode(ESelectionMode::Multi)
			.OnGenerateRow(this, &SSandboxListView::OnGenerateRow)
			.OnMouseButtonDoubleClick(this, &SSandboxListView::OnItemDoubleClicked)
			.HeaderRow(MakeHeaderRow())
			.OnSelectionChanged(this, &SSandboxListView::OnSelectionChanged)
			.OnContextMenuOpening(this, &SSandboxListView::OnContextMenuOpening)
		]

		+SOverlay::Slot()
		.Padding(0, 20, 0, 0) // To ensure the panel is below the row header.
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoSandboxes", "Sandboxes you create appear here."))
				.Visibility(this, &SSandboxListView::GetOverlayTextVisibility)
			]
		]
	];
	
	ControlsViewModel->OnCreationWorkflowStarted().AddSP(this, &SSandboxListView::OnCreationWorkflowStarted);
	ControlsViewModel->OnCreationWorkflowEnded().AddSP(this, &SSandboxListView::OnCreationWorkflowEnded);
	if (FSandboxCreationWorkflow* CurrentWorkflow = ControlsViewModel->GetCurrentCreationWorkflow())
	{
		OnCreationWorkflowStarted(*CurrentWorkflow);
	}
	
	ListViewModel->OnItemsChanged().AddSP(this, &SSandboxListView::OnItemsChanged);
	
	// Finally, refresh the UI to display all the current items.
	OnItemsChanged();
}

void SSandboxListView::ForEachSelectedItem(TFunctionRef<EBreakBehavior(const FString& InRootPath)> InCallback) const
{
	// We should report no selection while we're in some kind of operation / workflow.
	if (DummyItem)
	{
		return;
	}
	
	for (const TSharedPtr<FSandboxListItem>& Item : ListView->GetSelectedItems())
	{
		if (InCallback(Item->SandboxInfo.SandboxRoot) == EBreakBehavior::Break)
		{
			break;
		}
	}
}

TArray<FString> SSandboxListView::GetSelectedSandboxRootPaths() const
{
	TArray<FString> Result;
	Algo::Transform(ListView->GetSelectedItems(), Result, [](const TSharedPtr<FSandboxListItem>& Item)
	{
		return Item->SandboxInfo.SandboxRoot;
	});
	return Result;
}

TOptional<FString> SSandboxListView::GetSelectedItem() const
{
	return NumSelected() == 1 ? ListView->GetSelectedItems()[0]->SandboxInfo.SandboxRoot : TOptional<FString>{};
}

int32 SSandboxListView::NumSelected() const
{
	return DummyItem ? 0 : ListView->GetNumItemsSelected();
}

TSharedRef<SHeaderRow> SSandboxListView::MakeHeaderRow()
{
	const TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		.CanSelectGeneratedColumn(true);

	for (FName ColumnId : ListViewModel->GetDisplayedColumns())
	{
		if (const TSharedRef<ISandboxColumnWidgetFactory>* Factory = ColumnFactoryMap.Find(ColumnId))
		{
			HeaderRow->AddColumn(Factory->Get().MakeColumnArguments());
		}
	}

	return HeaderRow;
}

TSharedRef<SWidget> SSandboxListView::MakeColumnVisibilityMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(TEXT("Columns"), LOCTEXT("Columns", "Columns"));
	{
		// Add checkbox for each column
		for (const auto& Pair : ColumnFactoryMap)
		{
			const FName ColumnId = Pair.Key;

			// Get column label
			SHeaderRow::FColumn::FArguments ColumnArgs = Pair.Value->MakeColumnArguments();
			const FText ColumnLabel = ColumnArgs._DefaultLabel.Get();

			MenuBuilder.AddMenuEntry(
				ColumnLabel,
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, ColumnId]()
					{
						const bool bIsVisible = ListViewModel->IsColumnVisible(ColumnId);
						ListViewModel->SetColumnVisibility(ColumnId, !bIsVisible);
					}),
					FCanExecuteAction::CreateLambda([ColumnId]()
					{
						// Name column cannot be hidden
						return ColumnId != NameSandboxColumn;
					}),
					FIsActionChecked::CreateLambda([this, ColumnId]()
					{
						return ListViewModel->IsColumnVisible(ColumnId);
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> SSandboxListView::OnGenerateRow(TSharedPtr<FSandboxListItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InItem);
	const FMargin Padding = FSandboxedEditingStyle::Get().GetMargin("SandboxedEditing.Browser.RowPadding");
	
	const bool bIsDummyItem = DummyItem && InItem == DummyItem->Item; 
	if (bIsDummyItem)
	{
		switch (DummyItem->Type)
		{
		case EDummySandboxItemType::CreateSandbox: return SNew(SCreateSandboxListRow, InOwnerTable)
			.Padding(Padding)
			.Workflow_Lambda([this]{ return ControlsViewModel->GetCurrentCreationWorkflow(); });
		default: break;
		}
	}
	
	return SNew(SSandboxListRow, InItem.ToSharedRef(), InOwnerTable)
		.ColumnFactories(ColumnFactoryMap)
		.Padding(Padding)
		.IsSelected_Lambda([this, InItem]{ return ListView->IsItemSelected(InItem); })
		.HighlightText(HighlightTextAttr);
}

void SSandboxListView::OnItemDoubleClicked(TSharedPtr<FSandboxListItem> InItem)
{
	if (DummyItem && DummyItem->Item == InItem)
	{
		return;
	}
	
	ControlsViewModel->LoadSandbox(InItem->SandboxInfo.SandboxRoot);
}

TSharedPtr<SWidget> SSandboxListView::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, ContextMenuCommandList);
	
	TArray<TSharedPtr<FSandboxListItem>> Selection = ListView->GetSelectedItems();
	MenuBuilder.BeginSection(TEXT("Sandbox"), LOCTEXT("Sandbox", "Sandbox"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	
		if (Selection.Num() == 1)
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		}
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection(TEXT("Transfer"), LOCTEXT("Transfer", "Transfer"));
	{
		FBrowserCommands& BrowserCommands = FBrowserCommands::Get();
		if (!Selection.IsEmpty())
		{
			MenuBuilder.AddMenuEntry(BrowserCommands.ExportSandboxes);
		}
		MenuBuilder.AddMenuEntry(BrowserCommands.ImportSandboxes);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SSandboxListView::OnItemsChanged()
{
	const TArray<TSharedPtr<FSandboxListItem>>& NewItems = ListViewModel->GetItems();
	const bool bHasDummyItem = DummyItem.IsSet();
	
	Items.Empty(NewItems.Num() + static_cast<int32>(bHasDummyItem));
	if (bHasDummyItem)
	{
		Items.Add(DummyItem->Item);
	}
	Items.Append(NewItems);
	
	ListView->RequestListRefresh();
}

void SSandboxListView::OnCreationWorkflowStarted(FSandboxCreationWorkflow& InWorkflow)
{
	if (ensure(!DummyItem))
	{
		DummyItem.Emplace(MakeShared<FSandboxListItem>(FSandboxInfo{}), EDummySandboxItemType::CreateSandbox);
		Items.Insert(DummyItem->Item, 0);
		ListView->RequestListRefresh();
	}
}

void SSandboxListView::OnCreationWorkflowEnded()
{
	if (ensure(DummyItem))
	{
		Items.RemoveSingle(DummyItem->Item);
		DummyItem.Reset();
		ListView->RequestListRefresh();
	}
}

void SSandboxListView::OnSelectionChanged(TSharedPtr<FSandboxListItem> InItem, ESelectInfo::Type) const
{
	OnSelectedSandboxChangedDelegate.ExecuteIfBound(
		InItem ? TOptional<FString>(InItem->SandboxInfo.SandboxRoot) : TOptional<FString>()
		);
}

EVisibility SSandboxListView::GetOverlayTextVisibility() const
{
	return ListViewModel->GetItems().IsEmpty() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}
}

#undef LOCTEXT_NAMESPACE
