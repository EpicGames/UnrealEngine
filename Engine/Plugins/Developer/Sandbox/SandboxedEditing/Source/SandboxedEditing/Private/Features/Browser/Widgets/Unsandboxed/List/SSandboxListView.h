// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DummySandboxItem.h"
#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Features/Browser/ViewModels/SandboxCreationWorkflow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class ITableRow;
class SHeaderRow;
class STableViewBase;
template <typename ItemType> class SListView;

namespace UE::SandboxedEditing
{
class FSandboxControlsViewModel;
class FSandboxListItem;
class FSandboxListViewModel;
enum class EBreakBehavior : uint8;

DECLARE_DELEGATE_OneParam(FSandboxDelegate, TOptional<FString> /*InSandboxRoot*/);

/** A list view containing all the sandboxes. */
class SSandboxListView : public SCompoundWidget
{
public:
	
	struct FViewModels
	{
		TSharedRef<FSandboxListViewModel> ListViewModel;
		TSharedRef<FSandboxControlsViewModel> ControlsViewModel;
	};
	
	SLATE_BEGIN_ARGS(SSandboxListView){}
		/** The text to highlight in the columns. */
		SLATE_ATTRIBUTE(FText, HighlightText)
		
		/** Used to build the columns. */
		SLATE_ARGUMENT(FSandboxColumnFactoryMap, ColumnFactories)

		/** The command list to use in the context menu */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, ContextMenuCommandList)
		
		/** Invoked when the selected sandbox changes. */
		SLATE_EVENT(FSandboxDelegate, OnSelectedSandboxChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FViewModels& InViewModels);
	
	/** Enumerates every selected sandbox. */
	void ForEachSelectedItem(TFunctionRef<EBreakBehavior(const FString& InRootPath)> InCallback) const;
	/** @return Array containing the root path of every sandboxed selected in the list view. */
	TArray<FString> GetSelectedSandboxRootPaths() const;
	
	/** @return The sandbox that is selected if exactly one item is selected. */
	TOptional<FString> GetSelectedItem() const;
	
	/** @return Number of items selected */
	int32 NumSelected() const;
	
private:
	
	/** Knows about the items that are displayed in the browser. */
	TSharedPtr<FSandboxListViewModel> ListViewModel;
	/** Responds to creation of FSandboxCreationWorkflow by adding a dummy row into the table view. */
	TSharedPtr<FSandboxControlsViewModel> ControlsViewModel;
	
	/** Used to build the columns. */
	FSandboxColumnFactoryMap ColumnFactoryMap;
	/** The text to highlight in the columns. */
	TAttribute<FText> HighlightTextAttr;
	/** The command list to use in the context menu */
	TSharedPtr<FUICommandList> ContextMenuCommandList;
	
	/** Displays the items. */
	TSharedPtr<SListView<TSharedPtr<FSandboxListItem>>> ListView;
	/** The items displayed in the ListView. A copy of the items in ListViewModel plus any dummy row. */
	TArray<TSharedPtr<FSandboxListItem>> Items;
	
	/**
	 * Dummy item used to make the ListView display an additional row, e.g. for creating a new sandbox.
	 * Usually inserted at the beginning of items while in a creation workflow. It is inserted first so it's displayed on the top.
	 */
	TOptional<FDummySandboxItemInfo> DummyItem;
	
	/** Invoked when the selected sandbox changes. */
	FSandboxDelegate OnSelectedSandboxChangedDelegate;
	
	/** @return The header row to display at the top of the table */
	TSharedRef<SHeaderRow> MakeHeaderRow();

	/** @return Column visibility menu content */
	TSharedRef<SWidget> MakeColumnVisibilityMenu();

	/** @return Widget for the items */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FSandboxListItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	/** Loads the clicked sandbox */
	void OnItemDoubleClicked(TSharedPtr<FSandboxListItem> InItem);
	/** @return Context menu to show */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Invoked when the list needs to be regenerated. */
	void OnItemsChanged();
	
	void OnCreationWorkflowStarted(FSandboxCreationWorkflow& InWorkflow);
	void OnCreationWorkflowEnded();
	
	void OnSelectionChanged(TSharedPtr<FSandboxListItem> InItem, ESelectInfo::Type) const;
	
	EVisibility GetOverlayTextVisibility() const;
};
}

