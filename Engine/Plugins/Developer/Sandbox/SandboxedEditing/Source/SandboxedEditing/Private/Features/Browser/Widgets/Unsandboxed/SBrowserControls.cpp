// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBrowserControls.h"

#include "ISettingsModule.h"
#include "SandboxedEditingStyle.h"
#include "Features/Browser/Commands/BrowserCommands.h"
#include "Features/Browser/ViewModels/SandboxControlsViewModel.h"
#include "Filters/CustomTextFilters.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Utils/WidgetUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SBrowserControls"

namespace UE::SandboxedEditing
{
void SBrowserControls::Construct(const FArguments& InArgs, const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel)
{
	ControlsViewModel = InControlsViewModel;
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			BuildButtonBar(InArgs)
		]

		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]
		
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(1.f)
		[
			CreateOpenSettingsButton()
		]
	];
}

TSharedRef<SWidget> SBrowserControls::BuildButtonBar(const FArguments& InArgs)
{
	FToolBarBuilder RowBuilder(nullptr, FMultiBoxCustomization::None);
	
	const TSharedPtr<FUICommandList>& CommandList = InArgs._CommandList;
	FGenericCommands& GenericCommands = FGenericCommands::Get();
	FBrowserCommands& Commands = FBrowserCommands::Get();
	
	RowBuilder.AddWidget(
		MakeIconButtonByCommand(
			FSandboxedEditingStyle::Get().GetBrush("SandboxedEditing.Browser.NewSandbox"),
			CommandList,
			Commands.CreateNewSandbox, LOCTEXT("CreateNew", "Create a new sandbox")
		));
	
	RowBuilder.AddSeparator();
	
	RowBuilder.AddWidget(
		MakeIconButtonByCommand(
			FSandboxedEditingStyle::Get().GetBrush("SandboxedEditing.Browser.DeleteSandbox"),
			CommandList,
			GenericCommands.Delete, LOCTEXT("Delete", "Delete sandbox")
		));
	
	FBrowserCommands& BrowserCommands = FBrowserCommands::Get();
	RowBuilder.AddWidget(
		MakeIconButtonByCommand(
			FSandboxedEditingStyle::Get().GetBrush("SandboxedEditing.Browser.ExportSandboxes"),
			CommandList,
			BrowserCommands.ExportSandboxes
		));
	RowBuilder.AddWidget(
		MakeIconButtonByCommand(
			FSandboxedEditingStyle::Get().GetBrush("SandboxedEditing.Browser.ImportSandboxes"),
			CommandList,
			BrowserCommands.ImportSandboxes
		));
	
	return RowBuilder.MakeWidget();
}
}

#undef LOCTEXT_NAMESPACE