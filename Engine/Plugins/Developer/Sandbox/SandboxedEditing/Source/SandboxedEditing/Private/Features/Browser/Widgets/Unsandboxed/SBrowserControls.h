// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FSandboxControlsViewModel;
class FSandboxListItem;
class FSandboxListViewModel;

/** Controls widget to place on the top section of the browser. */
class SBrowserControls : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SBrowserControls){}
		/** The command list to bind selection-based commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSandboxControlsViewModel>& InControlsViewModel);
	
private:
	
	/** View model for mutating */
	TSharedPtr<FSandboxControlsViewModel> ControlsViewModel;
	
	/** Builds the button bar */
	TSharedRef<SWidget> BuildButtonBar(const FArguments& InArgs);
};
}

