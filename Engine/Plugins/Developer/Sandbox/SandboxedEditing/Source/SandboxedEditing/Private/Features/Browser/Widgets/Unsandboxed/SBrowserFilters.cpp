// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBrowserFilters.h"

#include "Features/Browser/ViewModels/List/FilterSandboxViewModel.h"
#include "Internationalization/Text.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SSandboxFilters"

namespace UE::SandboxedEditing
{
void SBrowserFilters::Construct(const FArguments& InArgs, const TSharedRef<FFilterSandboxViewModel>& InFilterSandboxViewModel)
{
	FilterSandboxViewModel = InFilterSandboxViewModel;
	
	ChildSlot
	[
		SNew(SSearchBox)
		.HintText(LOCTEXT("SearchHint", "Search sandboxes"))
		.OnTextChanged(this, &SBrowserFilters::OnSearchTextChanged)
		.OnTextCommitted(this, &SBrowserFilters::OnSearchTextCommitted)
		.DelayChangeNotificationsWhileTyping(true)
	];
}

void SBrowserFilters::OnSearchTextChanged(const FText& Text) const
{
	FilterSandboxViewModel->SetSearchText(Text);
}

void SBrowserFilters::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type)
{
	if (!InFilterText.EqualTo(FilterSandboxViewModel->GetSearchText()))
	{
		OnSearchTextChanged(InFilterText);
	}
}
}

#undef LOCTEXT_NAMESPACE