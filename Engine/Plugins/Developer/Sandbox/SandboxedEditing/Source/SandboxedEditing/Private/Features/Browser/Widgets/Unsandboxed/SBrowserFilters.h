// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SandboxedEditing
{
class FFilterSandboxViewModel;

class SBrowserFilters : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SBrowserFilters) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FFilterSandboxViewModel>& InFilterSandboxViewModel);
	
private:
	
	/** Knows how to filter sandbox items. */
	TSharedPtr<FFilterSandboxViewModel> FilterSandboxViewModel;
	
	void OnSearchTextChanged(const FText& Text) const;
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type Arg);
};
}

