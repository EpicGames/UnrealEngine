// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBrowserSandboxDetails.h"

#include "Components/VerticalBox.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumnRegistry.h"
#include "Features/Browser/ViewModels/FileState/Models/UnloadedSandboxFileStateViewModel.h"
#include "Features/Browser/Views/Menu/FileStateMenuUtils.h"
#include "Features/Browser/Widgets/Shared/FileState/SFilterableFileStateListView.h"
#include "Features/Browser/Widgets/Shared/SSandboxDescription.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Text.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBrowserSandboxDetails"

namespace UE::SandboxedEditing
{
namespace SandboxDetails
{
static TSharedRef<SWidget> MakeMessageWidget(TAttribute<FText> InMessageAttr)
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(STextBlock)
			.Text(InMessageAttr)
			.Justification(ETextJustify::Center)
		];
}
}
void SBrowserSandboxDetails::Construct(
	const FArguments& InArgs, 
	const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel,
	const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
	)
{
	NumSelectedAttr = InArgs._NumSelected;
	
	ChildSlot
	[
		MakeWidgetSwitcherContent(InArgs, InMetaDataViewModel, InFileActionsViewModel, InFilterViewModel)
	];
	
	FileStateCommandBindings = MakeUnique<FFileStateCommandBindings>(
		InArgs._SandboxPath, 
		FileStateMenu::MakeSelectedFilesAttribute(FileActionsView->GetListView().ToSharedRef())
		);
}

TSharedRef<SWidget> SBrowserSandboxDetails::MakeWidgetSwitcherContent(
	const FArguments& InArgs, 
	const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel, 
	const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
	)
{
	return SNew(SBox)
		.MinDesiredHeight(120)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &SBrowserSandboxDetails::GetWidgetIndex)

			+SWidgetSwitcher::Slot()
			[
				MakeDetailContent(InArgs, InMetaDataViewModel, InFileActionsViewModel, InFilterViewModel)
			]
			
			+SWidgetSwitcher::Slot()
			[
				SandboxDetails::MakeMessageWidget(LOCTEXT("NoneSelected", "Select a sandbox to view details."))
			]

			+SWidgetSwitcher::Slot()
			[
				SandboxDetails::MakeMessageWidget(LOCTEXT("TooManySelected", "Select only one sandbox to view details."))
			]
		];
}

TSharedRef<SWidget> SBrowserSandboxDetails::MakeDetailContent(
	const FArguments& InArgs,
	const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel, 
	const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
	const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
	)
{
	return SNew(SVerticalBox)
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(SExpandableArea)
			.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Description.Label", "Description"))
				.ToolTipText(LOCTEXT("Description.ToolTip", "Allows you to edit the sandbox description"))
			]
			.BodyContent()
			[
				SNew(SSandboxDescription, InMetaDataViewModel)
				.SandboxPath(InArgs._SandboxPath)
			]
		]
		
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(5.f)
		[
			SNew(SExpandableArea)
			.BodyBorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Content.Label", "File Changes"))
				.ToolTipText(LOCTEXT("Content.ToolTip", "Shows the files changed by this sandbox"))
			]
			.BodyContent()
			[
				SAssignNew(FileActionsView, SFilterableFileStateListView, InFileActionsViewModel, InFilterViewModel)
                .ColumnFactories(InArgs._ColumnFactories)
				.OnContextMenuOpening(this, &SBrowserSandboxDetails::MakeFileChangeContextMenu)
			]
		];
}

int32 SBrowserSandboxDetails::GetWidgetIndex() const
{
	constexpr int32 MainContent = 0;
	constexpr int32 NoneSelected = 1;
	constexpr int32 TooManySelected = 2;
	
	const bool bIsSet = NumSelectedAttr.IsSet() || NumSelectedAttr.IsBound();
	if (!bIsSet)
	{
		return MainContent;
	}
	
	const int32 NumSelected = NumSelectedAttr.Get();
	if (NumSelected == 1)
	{
		return MainContent;
	}
	
	if (NumSelected == 0)
	{
		return NoneSelected;
	}
	return TooManySelected;
}

TSharedPtr<SWidget> SBrowserSandboxDetails::MakeFileChangeContextMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	
	FileStateCommandBindings->AppendMenu(MenuBuilder);
	
	return MenuBuilder.MakeWidget();
}
}

#undef LOCTEXT_NAMESPACE