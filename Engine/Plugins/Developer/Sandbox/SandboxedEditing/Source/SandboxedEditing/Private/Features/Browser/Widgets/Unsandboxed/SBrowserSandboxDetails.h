// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/Commands/FileStateActions/FileStateCommandBindings.h"
#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SandboxedEditing
{
class SFilterableFileStateListView;
class FFilterFileStateViewModel;
class FSandboxMetaDataViewModel;
class FUnloadedSandboxFileStateViewModel;

/** Shows details for a sandbox in the browser widget. Handles showing text such as "select sandbox to view details", etc. */
class SBrowserSandboxDetails : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SBrowserSandboxDetails){}
		/** Path to the selected sandbox. */
		SLATE_ATTRIBUTE(TOptional<FString>, SandboxPath)
		/** The number of selected sandboxes. Relevant for the content displayed. */
		SLATE_ATTRIBUTE(int32, NumSelected)
		
		/** Used to build the columns for the file actions list. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, ColumnFactories)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, 
		const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel,
		const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
		);
	
private:
	
	/** Gets the number of selected items. Determines the content displayed. */
	TAttribute<int32> NumSelectedAttr;
	
	/** List of file actions being displayed. */
	TSharedPtr<SFilterableFileStateListView> FileActionsView;
	
	/** Common entries for FileActionWidget. */
	TUniquePtr<FFileStateCommandBindings> FileStateCommandBindings;
	
	/** @return The widget switcher that switches between the different messages and actual details content. */
	TSharedRef<SWidget> MakeWidgetSwitcherContent(
		const FArguments& InArgs, 
		const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel,
		const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
	);
	/** @return Content to display in the widget switcher */
	TSharedRef<SWidget> MakeDetailContent(
		const FArguments& InArgs, 
		const TSharedRef<FSandboxMetaDataViewModel>& InMetaDataViewModel, 
		const TSharedRef<FUnloadedSandboxFileStateViewModel>& InFileActionsViewModel,
		const TSharedRef<FFilterFileStateViewModel>& InFilterViewModel
		);
	
	/** @return Index for widget switcher to display. Determined by how many items are selected. */
	int32 GetWidgetIndex() const;

	TSharedPtr<SWidget> MakeFileChangeContextMenu();
};
}
