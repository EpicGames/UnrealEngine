// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxedEditingBrowser.h"

#include "Features/Browser/Commands/BrowserCommandBindings.h"
#include "Features/Browser/ViewModels/FileState/Models/UnloadedSandboxFileStateViewModel.h"
#include "Features/Browser/ViewModels/List/FilterSandboxViewModel.h"
#include "Framework/Commands/GenericCommands.h"
#include "GenericBrowser/SSessionBrowserFrame.h"
#include "List/SSandboxListView.h"
#include "Misc/MessageDialog.h"
#include "SBrowserControls.h"
#include "SBrowserFilters.h"
#include "SBrowserSandboxDetails.h"
#include "Features/Browser/Commands/BrowserCommands.h"
#include "Utils/BreakBehavior.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSandboxedEditingBrowser"

namespace UE::SandboxedEditing
{
void SSandboxedEditingBrowser::Construct(const FArguments& InArgs, const FViewModels& InViewModels)
{
	ControlsViewModel = InViewModels.ControlsViewModel;
	UnloadedSandboxFileStateViewModel = InViewModels.UnloadedSandboxFileStateViewModel;
	CommandList = InArgs._CommandList;
	
	const SSandboxListView::FViewModels ListViewModels { InViewModels.ListViewModel, InViewModels.ControlsViewModel };
	const TSharedRef<SSandboxListView> ListViewRef = SNew(SSandboxListView, ListViewModels)
		.ColumnFactories(InArgs._SandboxColumnFactories)
		.HighlightText(InViewModels.FilterSandboxViewModel, &FFilterSandboxViewModel::GetSearchText)
		.ContextMenuCommandList(CommandList)
		.OnSelectedSandboxChanged(this, &SSandboxedEditingBrowser::OnSelectedSandboxChanged);
	ListView = ListViewRef;
	
	ChildSlot
	[
		SNew(SSessionBrowserFrame)
		.ControlContent() [ SNew(SBrowserControls, InViewModels.ControlsViewModel).CommandList(InArgs._CommandList) ]
		.SearchContent() [ SNew(SBrowserFilters, InViewModels.FilterSandboxViewModel)]
		.SessionContent() [ ListViewRef ]
		.DetailsContent()
		[
			SNew(SBrowserSandboxDetails, 
				InViewModels.MetaDataViewModel, InViewModels.UnloadedSandboxFileStateViewModel, InViewModels.FilterUnloadedSandboxFileStateViewModel
				)
			.SandboxPath(ListViewRef, &SSandboxListView::GetSelectedItem)
			.NumSelected(ListViewRef, &SSandboxListView::NumSelected)
			.ColumnFactories(InArgs._FileActionsColumnFactories)
		]
	];
	
	BindCommands();
}

void SSandboxedEditingBrowser::BindCommands()
{
	if (!CommandList)
	{
		return;
	}
	
	FGenericCommands& GenericCommands = FGenericCommands::Get();
	CommandList->MapAction(
		GenericCommands.Delete, 
		FExecuteAction::CreateSP(this, &SSandboxedEditingBrowser::HandleDeleteSelection), 
		FCanExecuteAction::CreateSP(this, &SSandboxedEditingBrowser::CanDeleteSelection)
		);
	
	CommandList->MapAction(
		GenericCommands.Rename,
		FExecuteAction::CreateSP(this, &SSandboxedEditingBrowser::HandleRenameSelection)
		);
	
	FBrowserCommands& BrowserCommands = FBrowserCommands::Get();
	CommandList->MapAction(BrowserCommands.ExportSandboxes,
		FExecuteAction::CreateRaw(this, &SSandboxedEditingBrowser::HandleExportSandboxes),
		FCanExecuteAction::CreateRaw(this, &SSandboxedEditingBrowser::CanExportSandboxes)
		);
	CommandList->MapAction(BrowserCommands.ImportSandboxes, FExecuteAction::CreateRaw(this, &SSandboxedEditingBrowser::HandleImportSandboxes));
}

void SSandboxedEditingBrowser::HandleDeleteSelection() const
{
	if (CanDeleteSelection())
	{
		const EAppReturnType::Type Result = FMessageDialog::Open(
			EAppMsgType::YesNo, 
			FText::Format(LOCTEXT("Delete.MessageFmt", "Are you sure you want to delete {0} {0}|plural(one=sandbox,other=sandboxes)?"), ListView->NumSelected()),
			FText::Format(LOCTEXT("Delete.TitleFmt", "Delete {0}|plural(one=sandbox,other=sandboxes)"), ListView->NumSelected())
			);
		if (Result != EAppReturnType::Yes)
		{
			return;
		}
		
		ListView->ForEachSelectedItem([this](const FString& InRootPath)
		{
			ControlsViewModel->DeleteSandbox(InRootPath);
			return EBreakBehavior::Continue;
		});
	}
}

bool SSandboxedEditingBrowser::CanDeleteSelection() const
{
	if (ListView->NumSelected() == 0)
	{
		return false;
	}
	
	bool bCanDelete = true;
	ListView->ForEachSelectedItem([this, &bCanDelete](const FString& InRootPath)
	{
		bCanDelete &= ControlsViewModel->CanDeleteSandbox(InRootPath);
		return bCanDelete ? EBreakBehavior::Continue : EBreakBehavior::Break;
	});
	
	return bCanDelete;
}

void SSandboxedEditingBrowser::HandleRenameSelection() const
{
	if (const TOptional<FString> SandboxRoot = ListView->GetSelectedItem()
		; SandboxRoot && ControlsViewModel->CanRenameSandbox(*SandboxRoot))
	{
		ControlsViewModel->StartRenameWorkflow(*SandboxRoot);
	}
}

void SSandboxedEditingBrowser::HandleExportSandboxes()
{
	const TArray<FString> SandboxPaths = ListView->GetSelectedSandboxRootPaths();
	ControlsViewModel->StartExportWorkflow(SandboxPaths);
}

bool SSandboxedEditingBrowser::CanExportSandboxes()
{
	return ControlsViewModel->CanStartExportWorkflow() && !ListView->GetSelectedSandboxRootPaths().IsEmpty();
}

void SSandboxedEditingBrowser::HandleImportSandboxes()
{
	ControlsViewModel->StartImportWorkflow();
}

void SSandboxedEditingBrowser::OnSelectedSandboxChanged(TOptional<FString> InSandbox) const
{
	if (InSandbox)
	{
		UnloadedSandboxFileStateViewModel->SetContent(*InSandbox);
	}
	else
	{
		UnloadedSandboxFileStateViewModel->ClearContent();
	}
}
}

#undef LOCTEXT_NAMESPACE