// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/ViewModels/FileState/FileStateColumns.h"
#include "Features/Browser/ViewModels/List/SandboxColumns.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FFilterFileStateViewModel;
class FBrowserCommandBindings;
class FFilterSandboxViewModel;
class FSandboxControlsViewModel;
class FSandboxListViewModel;
class FSandboxMetaDataViewModel;
class FUnloadedSandboxFileStateViewModel;
class SSandboxListView;

/** Displays a list of sandboxes. Shows content only */
class SSandboxedEditingBrowser : public SCompoundWidget
{
public:
	
	struct FViewModels
	{
		TSharedRef<FFilterSandboxViewModel> FilterSandboxViewModel;
		TSharedRef<FSandboxListViewModel> ListViewModel;
		TSharedRef<FSandboxControlsViewModel> ControlsViewModel;
		TSharedRef<FSandboxMetaDataViewModel> MetaDataViewModel;
		TSharedRef<FUnloadedSandboxFileStateViewModel> UnloadedSandboxFileStateViewModel;
		TSharedRef<FFilterFileStateViewModel> FilterUnloadedSandboxFileStateViewModel;
	};

	SLATE_BEGIN_ARGS(SSandboxedEditingBrowser){}
		/** The command list to bind selection-based commands to. */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		/** Used to build the columns. */
		SLATE_ARGUMENT(FSandboxColumnFactoryMap, SandboxColumnFactories)
		/** Used to build the columns for the file actions list. */
		SLATE_ARGUMENT(FFileStateColumnFactoryMap, FileActionsColumnFactories)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FViewModels& InViewModels);
	
private:
	
	/** Used for implementing the commands we bind to. */
	TSharedPtr<FSandboxControlsViewModel> ControlsViewModel;
	/** When the selected sandbox changes, we notify this model. */
	TSharedPtr<FUnloadedSandboxFileStateViewModel> UnloadedSandboxFileStateViewModel;
	
	/** Optional command list for selection based commands, such as delete. */
	TSharedPtr<FUICommandList> CommandList;
	
	/** List view displaying the sandboxes. */
	TSharedPtr<SSandboxListView> ListView;


	/** Binds the commands only possible with UI, e.g. those that perform an action on the selection. */
	void BindCommands();
	
	/** Handles deleting the selection. */
	void HandleDeleteSelection() const;
	/** @return Whether the current selection can be deleted. */
	bool CanDeleteSelection() const;
	
	/** Renames the currently selected. Only does anything if exactly one item is selected. No-op, otherwise. */
	void HandleRenameSelection() const;
	
	/** Handles the command for exporting sandboxes. */
	void HandleExportSandboxes();
	/** @return Whether any sandboxes are selected in the browser. */
	bool CanExportSandboxes();
	
	/** Handles the command for importing sandboxes. */
	void HandleImportSandboxes();
	
	/** Notify the UnloadedSandboxFileStateViewModel when the selected sandbox changes. */
	void OnSelectedSandboxChanged(TOptional<FString> InSandbox) const;
};
}

