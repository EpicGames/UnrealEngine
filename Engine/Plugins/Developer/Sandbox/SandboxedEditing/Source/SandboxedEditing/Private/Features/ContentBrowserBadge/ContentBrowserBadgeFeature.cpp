// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserBadgeFeature.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Types/EBreakBehavior.h"
#include "Types/SandboxedFileChangeInfo.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::SandboxedEditing
{

FContentBrowserBadgeFeature::FContentBrowserBadgeFeature(TSharedRef<FSandboxSystemModel> InSandboxModel)
	: SandboxModel(InSandboxModel)
{
	// Subscribe to sandbox lifecycle events
	OnLoadSandboxHandle = SandboxModel->OnLoadSandbox().AddRaw(this, &FContentBrowserBadgeFeature::OnSandboxLoaded);
	OnLeaveSandboxHandle = SandboxModel->OnLeaveSandbox().AddRaw(this, &FContentBrowserBadgeFeature::OnSandboxLeft);

	// If a sandbox is already active, register immediately
	if (SandboxModel->HasActiveSandbox())
	{
		RegisterBadgeGenerators();
	}
}

FContentBrowserBadgeFeature::~FContentBrowserBadgeFeature()
{
	// Unregister badge generators if still registered
	UnregisterBadgeGenerators();

	// Unsubscribe from events
	if (OnLoadSandboxHandle.IsValid())
	{
		SandboxModel->OnLoadSandbox().Remove(OnLoadSandboxHandle);
	}
	if (OnLeaveSandboxHandle.IsValid())
	{
		SandboxModel->OnLeaveSandbox().Remove(OnLeaveSandboxHandle);
	}
}

void FContentBrowserBadgeFeature::RegisterBadgeGenerators()
{
	if (BadgeGeneratorHandle.IsValid())
	{
		// Already registered
		return;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// Create the badge generator with both icon and tooltip delegates
	FAssetViewExtraStateGenerator BadgeGenerator(
		FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FContentBrowserBadgeFeature::GenerateBadgeIcon),
		FOnGenerateAssetViewExtraStateIndicators::CreateRaw(this, &FContentBrowserBadgeFeature::GenerateBadgeTooltip)
	);

	BadgeGeneratorHandle = ContentBrowserModule.AddAssetViewExtraStateGenerator(BadgeGenerator);
}

void FContentBrowserBadgeFeature::UnregisterBadgeGenerators()
{
	if (!BadgeGeneratorHandle.IsValid())
	{
		return;
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.RemoveAssetViewExtraStateGenerator(BadgeGeneratorHandle);
	BadgeGeneratorHandle.Reset();
}

void FContentBrowserBadgeFeature::OnSandboxLoaded()
{
	RegisterBadgeGenerators();
}

void FContentBrowserBadgeFeature::OnSandboxLeft()
{
	UnregisterBadgeGenerators();
}

TSharedRef<SWidget> FContentBrowserBadgeFeature::GenerateBadgeIcon(const FAssetData& AssetData)
{
	const ESandboxChangeType ChangeType = GetAssetChangeType(AssetData);

	if (ChangeType == ESandboxChangeType::None)
	{
		// No badge for unchanged assets
		return SNullWidget::NullWidget;
	}

	// Choose icon and color based on change type
	const FSlateBrush* IconBrush = nullptr;
	FLinearColor BadgeColor = FLinearColor::White;

	switch (ChangeType)
	{
	case ESandboxChangeType::Added:
		IconBrush = FAppStyle::GetBrush("Icons.Plus");
		BadgeColor = FLinearColor(0.2f, 0.5f, 1.0f); // Blue for added files
		break;
	case ESandboxChangeType::Modified:
		IconBrush = FAppStyle::GetBrush("Icons.Edit");
		BadgeColor = FLinearColor(0.2f, 1.0f, 0.2f); // Green for modified files
		break;
	case ESandboxChangeType::Deleted:
		IconBrush = FAppStyle::GetBrush("Icons.Delete");
		BadgeColor = FLinearColor(1.0f, 0.2f, 0.2f); // Red for deleted files
		break;
	default:
		return SNullWidget::NullWidget;
	}

	return SNew(SImage)
		.Image(IconBrush)
		.ColorAndOpacity(BadgeColor);
}

TSharedRef<SWidget> FContentBrowserBadgeFeature::GenerateBadgeTooltip(const FAssetData& AssetData)
{
	const ESandboxChangeType ChangeType = GetAssetChangeType(AssetData);

	if (ChangeType == ESandboxChangeType::None)
	{
		// No tooltip for unchanged assets
		return SNullWidget::NullWidget;
	}

	// Create descriptive text based on change type
	FText TooltipText;
	switch (ChangeType)
	{
	case ESandboxChangeType::Added:
		TooltipText = FText::Format(
			NSLOCTEXT("SandboxedEditing", "BadgeTooltip_Added", "Added by sandbox: {0}"),
			FText::FromString(SandboxModel->GetActiveSandboxName())
		);
		break;
	case ESandboxChangeType::Modified:
		TooltipText = FText::Format(
			NSLOCTEXT("SandboxedEditing", "BadgeTooltip_Modified", "Modified by sandbox: {0}"),
			FText::FromString(SandboxModel->GetActiveSandboxName())
		);
		break;
	case ESandboxChangeType::Deleted:
		TooltipText = FText::Format(
			NSLOCTEXT("SandboxedEditing", "BadgeTooltip_Deleted", "Deleted in sandbox: {0}"),
			FText::FromString(SandboxModel->GetActiveSandboxName())
		);
		break;
	default:
		return SNullWidget::NullWidget;
	}

	return SNew(STextBlock)
		.Text(TooltipText);
}

FContentBrowserBadgeFeature::ESandboxChangeType FContentBrowserBadgeFeature::GetAssetChangeType(const FAssetData& AssetData) const
{
	using namespace FileSandboxCore;

	if (!SandboxModel->HasActiveSandbox())
	{
		return ESandboxChangeType::None;
	}

	// Get the package file path from the asset data
	FString PackageFilePath = FPackageName::LongPackageNameToFilename(
		AssetData.PackageName.ToString(),
		FPackageName::GetAssetPackageExtension()
	);

	// Convert to absolute path for comparison with sandbox change paths (which are absolute)
	PackageFilePath = FPaths::ConvertRelativePathToFull(PackageFilePath);

	// Check if this asset's file is in the change list by enumerating changes
	ESandboxChangeType ResultChangeType = ESandboxChangeType::None;

	SandboxModel->EnumerateFileChanges(
		SandboxModel->GetActiveSandboxPath(),
		[&PackageFilePath, &ResultChangeType](const FSandboxedFileChangeInfo& Change) -> UE::FileSandboxCore::EBreakBehavior
		{
			if (FPaths::IsSamePath(Change.Path, PackageFilePath))
			{
				switch (Change.Action)
				{
				case ESandboxFileChange::Added:
					ResultChangeType = ESandboxChangeType::Added;
					break;
				case ESandboxFileChange::Edited:
					ResultChangeType = ESandboxChangeType::Modified;
					break;
				case ESandboxFileChange::Removed:
					ResultChangeType = ESandboxChangeType::Deleted;
					break;
				default:
					break;
				}
				return UE::FileSandboxCore::EBreakBehavior::Break; // Found the file, stop searching
			}
			return UE::FileSandboxCore::EBreakBehavior::Continue;
		}
	);

	return ResultChangeType;
}

} // namespace UE::SandboxedEditing
