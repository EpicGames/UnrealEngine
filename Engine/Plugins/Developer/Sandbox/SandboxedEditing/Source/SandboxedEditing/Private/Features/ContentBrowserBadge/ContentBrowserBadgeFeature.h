// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class SWidget;
struct FAssetData;

namespace UE::SandboxedEditing
{
	
class FSandboxSystemModel;

/**
 * Feature that displays badges on Content Browser items for files modified in the active sandbox.
 * Registers badge generators with the Content Browser to show visual indicators for:
 * - Added files (new files created in sandbox)
 * - Modified files (existing files changed in sandbox)
 * - Deleted files (files removed in sandbox)
 */
class FContentBrowserBadgeFeature : public FNoncopyable
{
public:
	explicit FContentBrowserBadgeFeature(TSharedRef<FSandboxSystemModel> InSandboxModel);
	~FContentBrowserBadgeFeature();

private:
	/** Registers the badge generators with Content Browser */
	void RegisterBadgeGenerators();

	/** Unregisters the badge generators from Content Browser */
	void UnregisterBadgeGenerators();

	/** Called when a sandbox is loaded - registers badge generators */
	void OnSandboxLoaded();

	/** Called when a sandbox is left - unregisters badge generators */
	void OnSandboxLeft();

	/** Generates the badge icon widget for an asset */
	TSharedRef<SWidget> GenerateBadgeIcon(const FAssetData& AssetData);

	/** Generates the badge tooltip widget for an asset */
	TSharedRef<SWidget> GenerateBadgeTooltip(const FAssetData& AssetData);

	/** Checks if the given asset is modified in the active sandbox and returns the change type */
	enum class ESandboxChangeType : uint8
	{
		None,
		Added,
		Modified,
		Deleted
	};
	ESandboxChangeType GetAssetChangeType(const FAssetData& AssetData) const;

private:
	/** Model for interacting with the sandbox system */
	TSharedRef<FSandboxSystemModel> SandboxModel;

	/** Handle to the registered badge generator */
	FDelegateHandle BadgeGeneratorHandle;

	/** Delegate handles for sandbox lifecycle events */
	FDelegateHandle OnLoadSandboxHandle;
	FDelegateHandle OnLeaveSandboxHandle;
};

} // namespace UE::SandboxedEditing
