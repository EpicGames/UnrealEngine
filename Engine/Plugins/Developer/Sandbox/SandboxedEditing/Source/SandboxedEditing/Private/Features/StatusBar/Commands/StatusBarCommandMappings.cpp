// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatusBarCommandMappings.h"

#include "Editor.h"
#include "Features/StatusBar/Widgets/SCreateNewSandbox.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications.h"
#include "StatusBarCommands.h"

namespace UE::SandboxedEditing
{
FStatusBarCommandMappings::FStatusBarCommandMappings(const TSharedRef<FSandboxSystemModel>& InSandboxViewModel)
	: SandboxViewModel(InSandboxViewModel)
	, CommandList(MakeShared<FUICommandList>())
{
	FStatusBarCommands::Register();
	
	FStatusBarCommands& Commands = FStatusBarCommands::Get();

	CommandList->MapAction(
		Commands.OpenCreateNewSandboxDialog,
		FExecuteAction::CreateStatic(&ShowNewSandboxModal, InSandboxViewModel.ToSharedPtr()),
		FCanExecuteAction::CreateSP(InSandboxViewModel, &FSandboxSystemModel::CanCreateNewSandbox)
		);
	
	CommandList->MapAction(
		Commands.LeaveSandbox,
		FExecuteAction::CreateRaw(this, &FStatusBarCommandMappings::HandleLeaveSandbox),
		FCanExecuteAction::CreateSP(InSandboxViewModel, &FSandboxSystemModel::IsAllowedToLeaveSandbox)
		);

	CommandList->MapAction(
		Commands.PersistAll,
		FExecuteAction::CreateRaw(this, &FStatusBarCommandMappings::HandlePersistAll),
		FCanExecuteAction::CreateSP(InSandboxViewModel, &FSandboxSystemModel::CanPersistAllChanges)
	);

	CommandList->MapAction(
		Commands.DiscardAll,
		FExecuteAction::CreateRaw(this, &FStatusBarCommandMappings::HandleDiscardAll),
		FCanExecuteAction::CreateSP(InSandboxViewModel, &FSandboxSystemModel::CanRevertAllChanges)
	);
}

FStatusBarCommandMappings::~FStatusBarCommandMappings()
{
	FStatusBarCommands::Unregister();
}

void FStatusBarCommandMappings::HandleLeaveSandbox()
{
	if (GEditor && GEditor->IsPlaySessionInProgress())
	{
		ShowCannotLeaveDuringPlayMode();
		return;
	}

	const FString SandboxName = SandboxViewModel->GetActiveSandboxName();
	SandboxViewModel->LeaveSandbox();

	if (!SandboxName.IsEmpty()) // We expect this to be true
	{
		ShowLeftSandbox(*SandboxName);
	}
}

void FStatusBarCommandMappings::HandlePersistAll()
{
	SandboxViewModel->PersistAllChanges();
}

void FStatusBarCommandMappings::HandleDiscardAll()
{
	SandboxViewModel->RevertAllChanges();
}
}
