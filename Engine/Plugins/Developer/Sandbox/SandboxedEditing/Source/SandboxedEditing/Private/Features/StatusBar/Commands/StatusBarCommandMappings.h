// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FUICommandList;

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
	
/** Handles all commands for our app. Has the FUICommandList and binds all commands. */
class FStatusBarCommandMappings : public FNoncopyable
{
public:
	
	explicit FStatusBarCommandMappings(const TSharedRef<FSandboxSystemModel>& InSandboxViewModel);
	~FStatusBarCommandMappings();

	const TSharedRef<FUICommandList>& GetCommandList() const { return CommandList; }

private: 

	/** Required to handle the commands. */
	const TSharedRef<FSandboxSystemModel> SandboxViewModel;
	
	/** Holds our command bindings */
	const TSharedRef<FUICommandList> CommandList;
	
	void HandleLeaveSandbox();
	void HandlePersistAll();
	void HandleDiscardAll();
};
}

