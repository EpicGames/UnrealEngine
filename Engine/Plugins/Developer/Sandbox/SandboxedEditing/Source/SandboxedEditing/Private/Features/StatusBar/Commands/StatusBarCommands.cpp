// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatusBarCommands.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FSandboxedEditingCommands"

namespace UE::SandboxedEditing
{
FStatusBarCommands::FStatusBarCommands()
	: TCommands<FStatusBarCommands>
	(
		"SandboxedEditing.StatusBar",
		NSLOCTEXT("Contexts", "SandboxedEditing", "SandboxedEditing"),
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{}

void FStatusBarCommands::RegisterCommands()
{
	UI_COMMAND(OpenCreateNewSandboxDialog,
		"New Sandbox", "Creates a new sandbox for asset modifications.", EUserInterfaceActionType::Button, FInputChord()
		);
	UI_COMMAND(LeaveSandbox,
		"Leave Sandbox", "Leaves the active sandbox without persisting.", EUserInterfaceActionType::Button, FInputChord()
		);
	UI_COMMAND(PersistAll,
		"Persist All", "Persists all changes.", EUserInterfaceActionType::Button, FInputChord()
		);
	UI_COMMAND(DiscardAll,
		"Discard All", "Discards all changes.", EUserInterfaceActionType::Button, FInputChord()
		);
}
}

#undef LOCTEXT_NAMESPACE