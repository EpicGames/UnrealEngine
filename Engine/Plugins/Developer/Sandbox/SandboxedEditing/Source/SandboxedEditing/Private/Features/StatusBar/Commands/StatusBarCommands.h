// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::SandboxedEditing
{
class FStatusBarCommands : public TCommands<FStatusBarCommands>
{
public:

	FStatusBarCommands();

	//~ Begin TCommands Interface
	virtual void RegisterCommands() override;
	//~ End TCommands Interface

	/**
	 * Opens a dialog for creating a new sandbox.
	 * @note This is temporary UX - this will be removed when we implement the final UX.
	 */
	TSharedPtr<FUICommandInfo> OpenCreateNewSandboxDialog;
	
	/** Leaves the active sandbox without persisting. */
	TSharedPtr<FUICommandInfo> LeaveSandbox;
	
	/** Persists all changes. */
	TSharedPtr<FUICommandInfo> PersistAll;
	/** Discards all changes. */
	TSharedPtr<FUICommandInfo> DiscardAll;
};
}

