// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatusBarFeature.h"

#include "ToolMenus.h"
#include "Features/StatusBar/Widgets/SSandboxStatusBarWidget.h"

namespace UE::SandboxedEditing
{
namespace StatusBarDetail
{
static TSharedRef<SWidget> CreateStatusBarWidget(const TSharedRef<FSandboxSystemModel>& InViewModel, const FStatusBarCommandMappings& InCommandMappings)
{
	return SNew(SSandboxStatusBarWidget, InViewModel, InCommandMappings.GetCommandList());
}

static void RegisterStatusBar(const TSharedRef<FSandboxSystemModel>& InViewModel, const FStatusBarCommandMappings& InCommandMappings)
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("LevelEditor.StatusBar.ToolBar"));
	FToolMenuSection& AssetManagementSection = Menu->AddSection(
		TEXT("Asset Management"), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First)
		);

	AssetManagementSection.AddEntry(
		FToolMenuEntry::InitWidget(TEXT("SandboxStatusBar"), CreateStatusBarWidget(InViewModel, InCommandMappings), FText::GetEmpty(), true, false)
	);
}
}
	
FStatusBarFeature::FStatusBarFeature(const TSharedRef<FSandboxSystemModel>& InSandboxModel)
	: SandboxModel(InSandboxModel)
	, CommandMappings(SandboxModel)
{
	StatusBarDetail::RegisterStatusBar(SandboxModel, CommandMappings);
}
}
