// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/StatusBarCommandMappings.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
/** Manages extending the status bar for Sandboxed Editing. */
class FStatusBarFeature : public FNoncopyable
{
public:

	explicit FStatusBarFeature(const TSharedRef<FSandboxSystemModel>& InSandboxModel);
	
private:
	
	/** The shared sandbox model. */
	const TSharedRef<FSandboxSystemModel> SandboxModel;

	/** Handles all the input commands to this app. */
	const FStatusBarCommandMappings CommandMappings;
};
}
