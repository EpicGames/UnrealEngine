// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCreateNewSandbox.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Models/CreateSandboxArgs.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications.h"
#include "SPositiveActionButton.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCreateNewSandbox"

namespace UE::SandboxedEditing
{
void ShowNewSandboxModal(TSharedPtr<FSandboxSystemModel> InViewModel)
{
	if (!ensure(InViewModel.IsValid()))
	{
		return;
	}
	
	const TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateSandbox", "Create new sandbox"))
		.SizingRule(ESizingRule::Autosized)
		.ClientSize(FVector2D(600, 600))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	const TSharedRef<SCreateNewSandbox> CreateSandbox = SNew(SCreateNewSandbox, InViewModel.ToSharedRef()).ParentWindow(NewWindow);
	NewWindow->SetContent(
		CreateSandbox
	);

	FSlateApplication::Get().AddModalWindow(NewWindow, nullptr);
	if (!CreateSandbox->IsDialogConfirmed())
	{
		return;
	}

	const FCreateSandboxArgs Args = CreateSandbox->GetArgs();
	if (InViewModel->CreateNewSandbox(Args))
	{
		ShowCreatedSandbox(Args.Name);
	}
}
	
void SCreateNewSandbox::Construct(const FArguments& InArgs, const TSharedRef<FSandboxSystemModel>& InModel)
{
	ParentWindow = InArgs._ParentWindow;
	Model = InModel;
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.)
			[			
				SAssignNew(NameTextCtrl, SEditableTextBox)
				.HintText(LOCTEXT("Name.Hint", "Name"))
				.OnTextChanged(this, &SCreateNewSandbox::OnNameChanged)
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.)
			[
				CreateDescriptionInput()
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0.0f,0.0f,0.0f,5.0f)
			[
				CreateButtonArea()
			]
		]
	];
}

TSharedRef<SWidget> SCreateNewSandbox::CreateDescriptionInput()
{
	return SNew(SExpandableArea)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Description.Label", "Description"))
		]
		.BodyContent()
		[
			SNew(SBox)
			.MinDesiredHeight(120)
			.WidthOverride(320)
			[
				SAssignNew(DescriptionTextCtrl, SMultiLineEditableTextBox)
				.SelectAllTextWhenFocused(true)
				.AutoWrapText(true)
				.HintText(LOCTEXT("Description.Hint", "Optional description"))
			]
		];
}
	
TSharedRef<SWidget> SCreateNewSandbox::CreateButtonArea()
{
	return SNew(SUniformGridPanel)
		.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
		.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
		.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
		+ SUniformGridPanel::Slot(0, 0)
		[
			SNew(SPositiveActionButton)
			.IsEnabled(this, &SCreateNewSandbox::IsCreateButtonEnabled)
			.Text(LOCTEXT("ButtonCreate", "Create"))
			.ToolTipText(this, &SCreateNewSandbox::GetCreateButtonToolTipText)
			.OnClicked(this, &SCreateNewSandbox::OnCreateClicked)
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			SNew(SActionButton)
			.Text(LOCTEXT("CancelButton", "Cancel"))
			.OnClicked(this, &SCreateNewSandbox::OnCancelClicked)
		];
}

FReply SCreateNewSandbox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Pressing escape returns as if the user clicked cancel
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancelClicked();
	}

	return FReply::Unhandled();
}

FCreateSandboxArgs SCreateNewSandbox::GetArgs() const
{
	FCreateSandboxArgs Args;
	Args.Name = NameTextCtrl->GetText().ToString();
	Args.Description = DescriptionTextCtrl->GetText().ToString();
	return Args;
}

FReply SCreateNewSandbox::OnCreateClicked()
{
	bDialogConfirmed = true;
	auto ParentWindowPin = ParentWindow.Pin();
	if (ParentWindowPin.IsValid())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

bool SCreateNewSandbox::IsCreateButtonEnabled() const
{
	return Model->CanCreateNewSandbox(NameTextCtrl->GetText().ToString());
}

FText SCreateNewSandbox::GetCreateButtonToolTipText() const
{
	FText Reason;
	const bool bCanCreateSandbox = Model->CanCreateNewSandbox(NameTextCtrl->GetText().ToString(), &Reason);
	return bCanCreateSandbox
		? LOCTEXT("CanCreateToolTip", "Create sandbox")
		: Reason;
}

void SCreateNewSandbox::OnNameChanged(const FText& InNewName)
{
	FText Reason;
	const bool bCanCreateSandbox = Model->CanCreateNewSandbox(InNewName.ToString(), &Reason);
	
	if (bCanCreateSandbox) 
	{
		NameTextCtrl->SetError(FText::GetEmpty());
	}
	else
	{
		NameTextCtrl->SetError(Reason);
	}
}

FReply SCreateNewSandbox::OnCancelClicked()
{
	bDialogConfirmed = false;
	auto ParentWindowPin = ParentWindow.Pin();
	if (ParentWindowPin.IsValid())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}
}

#undef LOCTEXT_NAMESPACE