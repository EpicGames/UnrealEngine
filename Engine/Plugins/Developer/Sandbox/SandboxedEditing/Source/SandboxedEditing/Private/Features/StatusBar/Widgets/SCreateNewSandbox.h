// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SEditableTextBox;
class SMultiLineEditableTextBox;
class SErrorText;

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
struct FCreateSandboxArgs;
	
/** Shows a modal dialogue to the user for starting a new sandbox. */
void ShowNewSandboxModal(TSharedPtr<FSandboxSystemModel> InViewModel);
	
class SCreateNewSandbox : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCreateNewSandbox){}
		/** The parent window this widget is hosted in. */
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSandboxSystemModel>& InModel);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Get dialog result */
	bool IsDialogConfirmed() const { return bDialogConfirmed; }

	/** Get the new sandbox command from the dialog result. */
	FCreateSandboxArgs GetArgs() const;
		
private:
	
	/** Pointer to the parent modal window */
	TWeakPtr<SWindow> ParentWindow;

	/** Model use to determine whether a name is available. */
	TSharedPtr<FSandboxSystemModel> Model;
	
	TSharedPtr<SMultiLineEditableTextBox> DescriptionTextCtrl;
	TSharedPtr<SEditableTextBox> NameTextCtrl;
	
	/** Result confirmation */
	bool bDialogConfirmed = false;
	
	TSharedRef<SWidget> CreateDescriptionInput();
	TSharedRef<SWidget> CreateButtonArea();
	
	/** Called when the settings of the dialog are to be accepted*/
	FReply OnCreateClicked();
	/** Called to check if create button is enabled or not. */
	bool IsCreateButtonEnabled() const;
	/** @return Tooltip text to show on create button */
	FText GetCreateButtonToolTipText() const;

	/** Updates the error state when the name changes. */
	void OnNameChanged(const FText& InNewName);
	
	/** Called when the sandbox creation is abandoned. */
	FReply OnCancelClicked();
};
}

