// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSandboxStatusBarWidget.h"

#include "Editor.h"
#include "Features/StatusBar/Commands/StatusBarCommands.h"
#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications.h"
#include "SActionButton.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "ToolWidgetsSlateTypes.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSandboxStatusBarWidget"

namespace UE::SandboxedEditing
{
void SSandboxStatusBarWidget::Construct(
	const FArguments& InArgs,
	const TSharedRef<FSandboxSystemModel>& InViewModel, const TSharedRef<FUICommandList>& InCommandList
	)
{
	ViewModel = InViewModel;
	CommandList = InCommandList;
	
	ChildSlot
	[
		SNew(SComboButton)
		.ContentPadding(FMargin(6.0f, 0.0f))
		.MenuPlacement(MenuPlacement_AboveAnchor)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 10, 0)
			[
				SNew(STextBlock)
				.Text(this, &SSandboxStatusBarWidget::GetTitleText)
				.ToolTipText(LOCTEXT("AssetManagementStatusBarToolTip", "Tools for altering the state of assets."))
			]
		]
		.OnGetMenuContent(FOnGetContent::CreateRaw(this, &SSandboxStatusBarWidget::CreateStatusBarMenu))
	];
}

FText SSandboxStatusBarWidget::GetTitleText() const
{
	const FString SandboxName = ViewModel->GetActiveSandboxName();
	if (!SandboxName.IsEmpty())
	{
		FTextBuilder TextBuilder;
		TextBuilder.AppendLineFormat(LOCTEXT("LoadedSandbox", "{0} (Sandbox)"), FText::FromString(*SandboxName));
		return TextBuilder.ToText();
	}
	return LOCTEXT("StatusBarName", "Sandbox");
}

TSharedRef<SWidget> SSandboxStatusBarWidget::CreateStatusBarMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("StatusBar.ToolBar.SandboxedEditing", NAME_None, EMultiBoxType::Menu, false);

	{
		FToolMenuSection& Section = Menu->AddSection("SandboxedEditingSection", LOCTEXT("SandboxedEditing.Sandbox", "Sandbox"));

		const TAttribute<FText> LeaveToolTipOverride = TAttribute<FText>::CreateLambda([Model = ViewModel]
		{
			FText Reason;
			return Model->IsAllowedToLeaveSandbox(&Reason) ? FStatusBarCommands::Get().LeaveSandbox->GetDescription() : Reason;
		});
		const TAttribute<FText> OpenNewSandboxToolTipOverride = TAttribute<FText>::CreateLambda([Model = ViewModel]
		{
			if (!Model->CanCreateNewSandbox())
			{
				if (GEditor && GEditor->IsPlaySessionInProgress())
				{
					return LOCTEXT("SandboxEditing.CannotCreateSandboxDuringPlay",
								   "Cannot create a new sandbox while a play session is active.");
				}
				return LOCTEXT("SandboxEditing.CannotCreateSandbox",
							   "Cannot create a new sandbox because some another tool is using the sandbox and has locked access.");
			}
			return LOCTEXT("SandboxEditing.CreateNewSandbox", "Create a new sandbox with dialog.");
		});
		
		Section.AddMenuEntry(FStatusBarCommands::Get().OpenCreateNewSandboxDialog, {}, OpenNewSandboxToolTipOverride);
		Section.AddMenuEntry(FStatusBarCommands::Get().LeaveSandbox, {}, LeaveToolTipOverride);
		Section.AddMenuEntry(FStatusBarCommands::Get().PersistAll);
		Section.AddMenuEntry(FStatusBarCommands::Get().DiscardAll);

		BuildSandboxesMenu(Menu);
	}
	
	return UToolMenus::Get()->GenerateWidget("StatusBar.ToolBar.SandboxedEditing", FToolMenuContext(CommandList));
}

void SSandboxStatusBarWidget::BuildSandboxesMenu(UToolMenu* InMenuBuilder)
{
	FToolMenuSection& Section = InMenuBuilder->AddSection("Load Sandboxes", LOCTEXT("LoadSandboxes", "Load Sandbox"));
	Section.AddDynamicEntry("Existing Sandboxes",
		FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		TArray<FSandboxInfo> Sandboxes = ViewModel->GetKnownSandboxes();
		// Display them in alphabetical order
		Sandboxes.Sort();

		TArray<FString> DuplicateNameDetection;
		for (const FSandboxInfo& Sandbox : Sandboxes)
		{
			// Tool menus want unique names. It could happen that the user goes into the file system and manually edits the name entry in the JSON.
			const FName MenuName(
				FString::Printf(TEXT("%s_%s"),* Sandbox.Name, *FGuid::NewDeterministicGuid(Sandbox.SandboxRoot).ToString())
				);
			
			const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				MenuName,
				FText::FromString(Sandbox.Name),
				FText::FromString(Sandbox.Description),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, Sandbox]
					{
						if (ViewModel->LoadSandbox(Sandbox.SandboxRoot))
						{
							ShowLoadedSandbox(Sandbox.Name);
						}
						else
						{
							ShowFailedToLoadSandbox(Sandbox.Name);
						}
					}),
					FCanExecuteAction::CreateLambda([this, Path = Sandbox.SandboxRoot]{ return ViewModel->CanLoadSandbox(Path); }),
					FIsActionChecked::CreateLambda([this, Path = Sandbox.SandboxRoot]{ return ViewModel->IsActiveSandbox(Path); })
					),
				EUserInterfaceActionType::RadioButton
				);
			InSection.AddEntry(Entry);
		}
	}));
}
}

#undef LOCTEXT_NAMESPACE
