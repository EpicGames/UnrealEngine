// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolMenu.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/** Root widget that is placed in the status bar. */
class SSandboxStatusBarWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSandboxStatusBarWidget){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSandboxSystemModel>& InViewModel, const TSharedRef<FUICommandList>& InCommandList);

private:

	/** The model we're bound to. */
	TSharedPtr<FSandboxSystemModel> ViewModel;
	/** The commands needed to create the menu. */
	TSharedPtr<FUICommandList> CommandList;

	/** @return Text to display in the combo button content */
	FText GetTitleText() const;
	/** @return Content to show when combo button is opened */
	TSharedRef<SWidget> CreateStatusBarMenu();

	/** Fills the submenu showing the available sandboxes. */
	void BuildSandboxesMenu(UToolMenu* InMenuBuilder);
};
}
