// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShutdownCleanupHandler.h"

#include "Misc/CoreDelegates.h"
#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Logging/LogMacros.h"
#include "SandboxedEditingSettings.h"
#include "Types/GatheredFileChanges.h"

DEFINE_LOG_CATEGORY_STATIC(LogSandboxCleanup, Log, All);

namespace UE::SandboxedEditing
{

FShutdownCleanupHandler::FShutdownCleanupHandler(const TSharedRef<FSandboxSystemModel>& InModel)
	: SandboxModel(InModel)
{
	// Register for earliest shutdown hook
	ShutdownHandle = FCoreDelegates::OnEnginePreExit.AddRaw(this, &FShutdownCleanupHandler::OnEnginePreExit);
}

FShutdownCleanupHandler::~FShutdownCleanupHandler()
{
	if (ShutdownHandle.IsValid())
	{
		FCoreDelegates::OnEnginePreExit.Remove(ShutdownHandle);
	}
}

void FShutdownCleanupHandler::OnEnginePreExit()
{
	const USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();

	// Early exit if feature disabled or settings unavailable
	if (!Settings || !Settings->bDeleteEmptySandboxesOnExit)
	{
		return;
	}

	UE_LOGF(LogSandboxCleanup, Log, "Starting empty sandbox cleanup on shutdown");

	// Leave active sandbox first so we can check/delete it
	if (SandboxModel->HasActiveSandbox())
	{
		FString ActiveSandboxPath = SandboxModel->GetActiveSandboxPath();
		UE_LOGF(LogSandboxCleanup, Verbose, "Leaving active sandbox to check for cleanup: %ls", *ActiveSandboxPath);
		SandboxModel->LeaveSandbox();
	}

	// Get all known sandboxes
	TArray<FSandboxInfo> AllSandboxes = SandboxModel->GetKnownSandboxes();
	int32 DeletedCount = 0;

	for (const FSandboxInfo& Info : AllSandboxes)
	{
		// Check if sandbox is empty
		if (!HasFileChanges(Info.SandboxRoot))
		{
			UE_LOGF(LogSandboxCleanup, Log, "Deleting empty sandbox: %ls (Path: %ls)", *Info.Name, *Info.SandboxRoot);

			// Delete sandbox
			SandboxModel->DeleteSandbox(Info.SandboxRoot);
			DeletedCount++;
		}
	}

	if (DeletedCount > 0)
	{
		UE_LOGF(LogSandboxCleanup, Log, "Cleanup complete: Deleted %d empty sandbox(es)", DeletedCount);
	}
	else
	{
		UE_LOGF(LogSandboxCleanup, Log, "Cleanup complete: No empty sandboxes found");
	}
}

bool FShutdownCleanupHandler::HasFileChanges(const FString& SandboxRoot)
{
	// Gather file changes from the sandbox
	FileSandboxCore::FGatheredFileChanges Changes = SandboxModel->GatherFileChanges(SandboxRoot);

	// HasChanges() returns true if there are any created/modified/deleted files
	return Changes.HasChanges();
}

} // namespace UE::SandboxedEditing
