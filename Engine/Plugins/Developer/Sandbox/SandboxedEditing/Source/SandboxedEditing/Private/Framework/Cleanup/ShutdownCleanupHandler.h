// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/**
 * Handles cleanup of empty sandboxes on editor shutdown.
 * Checks all known sandboxes and deletes those with no file changes.
 */
class FShutdownCleanupHandler : public FNoncopyable
{
public:
	/**
	 * Constructor - registers for engine shutdown event
	 * @param InModel The sandbox system model to use for operations
	 */
	explicit FShutdownCleanupHandler(const TSharedRef<FSandboxSystemModel>& InModel);

	/** Destructor - unregisters delegates */
	~FShutdownCleanupHandler();

private:
	/** Called when engine is about to shut down */
	void OnEnginePreExit();

	/**
	 * Checks if a sandbox has any file changes
	 * @param SandboxRoot The root directory of the sandbox
	 * @return True if the sandbox has created/modified/deleted files
	 */
	bool HasFileChanges(const FString& SandboxRoot);

	/** The sandbox system model for all sandbox operations */
	TSharedRef<FSandboxSystemModel> SandboxModel;

	/** Delegate handle for cleanup */
	FDelegateHandle ShutdownHandle;
};

} // namespace UE::SandboxedEditing
