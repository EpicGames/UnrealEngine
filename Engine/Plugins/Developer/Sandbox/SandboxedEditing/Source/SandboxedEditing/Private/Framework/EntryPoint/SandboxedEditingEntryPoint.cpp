// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxedEditingEntryPoint.h"

#include "Data/SandboxMetaData.h"
#include "Framework/Models/SandboxTagging.h"
#include "ISandboxInstance.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "FSandboxedEditingEntryPoint"

namespace UE::SandboxedEditing
{
void FSandboxedEditingEntryPoint::SummonProviderUI()
{
	OnRequestSummonUIDelegate.Broadcast();
}

FText FSandboxedEditingEntryPoint::GetEntryPointLabel() const
{
	return LOCTEXT("EntryPoint", "Sandboxed Editing");
}

bool FSandboxedEditingEntryPoint::OwnsSandbox(const FileSandboxCore::ISandboxInstance& InSandbox) const
{
	return CanBeShownBySandboxedEditing(InSandbox.GetInitialMetaData());
}
}

#undef LOCTEXT_NAMESPACE