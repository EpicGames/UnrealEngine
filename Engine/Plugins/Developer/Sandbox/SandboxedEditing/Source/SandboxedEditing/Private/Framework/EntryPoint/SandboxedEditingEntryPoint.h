// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "EntryPoint/ISandboxEntryPoint.h"

namespace UE::SandboxedEditing
{
class FBrowserFeature;

/** Represents entry point for sandboxed in Sandboxed Editing. */
class FSandboxedEditingEntryPoint : public FileSandboxUI::ISandboxEntryPoint
{
public:
	
	//~ Begin ISandboxEntryPoint Interface
	virtual void SummonProviderUI() override;
	virtual FText GetEntryPointLabel() const override;
	virtual bool OwnsSandbox(const FileSandboxCore::ISandboxInstance& InSandbox) const override;
	//~ End ISandboxEntryPoint Interface
	
	/** Event invoked when the UI is supposed to be summoned. */
	FSimpleMulticastDelegate& OnRequestSummonUI() { return OnRequestSummonUIDelegate; }
	
private:
	
	/** Event invoked when the UI is supposed to be summoned. */
	FSimpleMulticastDelegate OnRequestSummonUIDelegate;
};
}

