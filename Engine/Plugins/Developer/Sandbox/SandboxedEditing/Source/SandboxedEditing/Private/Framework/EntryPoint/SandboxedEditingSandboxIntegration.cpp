// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxedEditingSandboxIntegration.h"

#include "EntryPoint/EntryPointWidgetFactory.h"
#include "EntryPoint/ISandboxEntryPointRegistry.h"
#include "IFileSandboxUIModule.h"
#include "SandboxedEditingEntryPoint.h"

namespace UE::SandboxedEditing
{
FSandboxedEditingSandboxIntegration::FSandboxedEditingSandboxIntegration()
	: EntryPoint(MakeShared<FSandboxedEditingEntryPoint>())
	, ExternalSandboxViewModel(FileSandboxUI::MakeExternalSandboxActiveViewModel(EntryPoint))
{
	using namespace UE::FileSandboxUI;
	ISandboxEntryPointRegistry& Registry = IFileSandboxUIModule::Get().GetEntryPointRegistry();
	Registry.RegisterEntryPoint(EntryPoint);
}

FSandboxedEditingSandboxIntegration::~FSandboxedEditingSandboxIntegration()
{
	using namespace UE::FileSandboxUI;
	if (IFileSandboxUIModule::IsAvailable())
	{
		ISandboxEntryPointRegistry& Registry = IFileSandboxUIModule::Get().GetEntryPointRegistry();
		Registry.UnregisterEntryPoint(EntryPoint);
	}
}
}
