// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::FileSandboxUI { class IExternalSandboxActiveViewModel; }

namespace UE::SandboxedEditing
{
class FSandboxedEditingEntryPoint;

/** Responsible for registering Sandboxed Editing with the sandbox system. */
class FSandboxedEditingSandboxIntegration : public FNoncopyable
{
public:
	
	FSandboxedEditingSandboxIntegration();
	~FSandboxedEditingSandboxIntegration();
	
	const TSharedRef<FSandboxedEditingEntryPoint>& GetEntryPoint() const { return EntryPoint; }
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel>& GetExternalSandboxViewModel() const { return ExternalSandboxViewModel; }
	
private:
	
	/** Represents Sandboxed Editing as entry point so it becomes discoverable by other editor systems. */
	const TSharedRef<FSandboxedEditingEntryPoint> EntryPoint;
	
	/** This view model is used to overlay a widget onto the browser when the engine is already sandboxed. */
	const TSharedRef<FileSandboxUI::IExternalSandboxActiveViewModel> ExternalSandboxViewModel;
};
}

