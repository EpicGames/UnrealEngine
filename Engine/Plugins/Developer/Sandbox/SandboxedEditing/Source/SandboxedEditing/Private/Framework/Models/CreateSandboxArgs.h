// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::SandboxedEditing
{
struct FCreateSandboxArgs
{
	FString Name;
	FString Description;
};
}