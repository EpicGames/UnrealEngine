// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Data/VersionInfo.h"
#include "Misc/DateTime.h"

namespace UE::SandboxedEditing
{
/** Describes a sandbox. */
struct FSandboxInfo
{
	/** Name the user has given the sandbox. May not be unique (but usually is). */
	FString Name;

	/** Description the user has given the sandbox. */
	FString Description;

	/** Path to the root of the sandbox directory. Unique for every sandbox item. */
	FString SandboxRoot;
	
	/** Info about when the sandbox was last modified in local time. */
	FDateTime LastModified;
	
	/** The version of the engine when this sandbox was created in. */
	FFileSandboxCore_EngineVersionInfo EngineVersion;

	FSandboxInfo() = default;
	explicit FSandboxInfo(
		FString InName, FString InDescription, FString InDirectoryPath,
		const FDateTime& InLastModified, FFileSandboxCore_EngineVersionInfo InEngineVersion
		)
		: Name(MoveTemp(InName))
		, Description(MoveTemp(InDescription))
		, SandboxRoot(MoveTemp(InDirectoryPath))
		, LastModified(InLastModified)
		, EngineVersion(MoveTemp(InEngineVersion))
	{}

	friend bool operator<(const FSandboxInfo& Left, const FSandboxInfo& Right)
	{
		return Left.Name < Right.Name;
	}
};
}
