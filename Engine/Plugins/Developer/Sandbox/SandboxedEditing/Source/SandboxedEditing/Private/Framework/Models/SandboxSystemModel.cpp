// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxSystemModel.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetViewUtils.h"
#include "Editor.h"
#include "Containers/UnrealString.h"
#include "Data/SandboxMetaData.h"
#include "Data/VersionInfo.h"
#include "Framework/Models/CreateSandboxArgs.h"
#include "Framework/Models/SandboxInfo.h"
#include "Framework/Notifications.h"
#include "IFileSandboxCoreModule.h"
#include "ISandboxInstance.h"
#include "ISandboxManager.h"
#include "ISandboxRepository.h"
#include "Interface/ISandboxLock.h"
#include "Misc/EngineVersion.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "SandboxedEditingSettings.h"
#include "SandboxTagging.h"
#include "Templates/Function.h"
#include "Types/EBreakBehavior.h"
#include "Types/Manager/DeleteSandboxByDirectoryArgs.h"
#include "Types/Manager/DeleteSandboxResult.h"
#include "Types/Manager/LoadSandboxByDirectoryArgs.h"
#include "Types/Manager/LoadSandboxError.h"
#include "Types/Manager/NewSandboxArgs.h"
#include "Types/Sandbox/RevertResult.h"
#include "Types/SandboxMetaInfo.h"
#include "Utils/PackageSandboxUtils.h"
#include "Utils/SandboxDirectoryUtils.h"

#define LOCTEXT_NAMESPACE "FSandboxSystemModel"

namespace UE::SandboxedEditing
{
FSandboxSystemModel::FSandboxSystemModel()
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	
	ISandboxManager& Manager = Module.GetSandboxManager();
	Manager.OnPostSandboxStartup().AddRaw(this, &FSandboxSystemModel::HandleLoadSandbox);
	Manager.OnPostSandboxShutdown().AddRaw(this, &FSandboxSystemModel::HandleLeaveSandbox);
	
	ISandboxRepository& Repository = Module.GetDefaultSandboxRepository();
	Repository.OnSandboxesChanged().AddRaw(this, &FSandboxSystemModel::HandleSandboxesChanged);
	Repository.OnSandboxMetaDataChanged().AddRaw(this, &FSandboxSystemModel::HandleMetaDataChanged);

	USandboxedEditingSettings::OnCustomDirectoryChanged.AddRaw(this, &FSandboxSystemModel::HandleCustomDirectoryChanged);
}

FSandboxSystemModel::~FSandboxSystemModel()
{
	using namespace FileSandboxCore;
	
	if (IFileSandboxCoreModule::IsAvailable())
	{
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		
		ISandboxManager& Manager = Module.GetSandboxManager();
		Manager.OnPostSandboxStartup().RemoveAll(this);
		Manager.OnPostSandboxShutdown().RemoveAll(this);
		if (ISandboxInstance* Instance = Manager.GetActiveSandboxInstance())
		{
			Instance->OnSandboxedFilesChanged().RemoveAll(this);
		}
		
		ISandboxRepository& Repository = Module.GetDefaultSandboxRepository();
		Repository.OnSandboxesChanged().RemoveAll(this);
		Repository.OnSandboxMetaDataChanged().RemoveAll(this);
	}

	USandboxedEditingSettings::OnCustomDirectoryChanged.RemoveAll(this);
}

bool FSandboxSystemModel::CreateNewSandbox(const FCreateSandboxArgs& InArgs)
{
	if (!CanCreateNewSandbox(InArgs.Name))
	{
		return false;
	}
	
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();

	if (!SandboxManager.LeaveSandbox())
	{
		return false;
	}

	FNewSandboxArgs Args(InArgs.Name, InArgs.Description);
	Args.SandboxBasePath = SandboxModel::GetBaseSandboxDirectory();
	MarkOwnedBySandboxedEditing(Args.MetaData);
	const bool bSuccess = SandboxManager.CreateNewSandbox(Args).HasValue();

	if (bSuccess)
	{
		NotifyIfBypassingRepository();
	}

	return bSuccess;
}

bool FSandboxSystemModel::CanCreateNewSandbox() const
{
	if (GEditor && GEditor->IsPlaySessionInProgress())
	{
		return false;
	}
	// If there is a leave lock in place it means that some other plugin has established a sandbox which would prevent us from
	// creating a new one.
	return !HasLeaveLock();
}
	
bool FSandboxSystemModel::CanCreateNewSandbox(const FString& InName, FText* Reason)
{
	const FString BaseDir = SandboxModel::GetBaseSandboxDirectory();
	FileSandboxCore::EDirectorySuitability Suitability = FileSandboxCore::DetermineDirectorySuitability(InName, BaseDir);

	const bool bIsSuitable = Suitability == FileSandboxCore::EDirectorySuitability::Suitable;
	if (Reason && !bIsSuitable)
	{
		*Reason = FileSandboxCore::FormatDirectorySuitabilityAsReason(Suitability, InName, BaseDir);
	}
	
	if (HasActiveSandbox() && !IsAllowedToLeaveSandbox(Reason))
	{
		return false;
	}
	
	return bIsSuitable;
}

bool FSandboxSystemModel::LoadSandbox(const FString& InRootDirectory)
{
	if (CanLoadSandbox(InRootDirectory))
	{
		using namespace FileSandboxCore;
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		ISandboxManager& SandboxManager = Module.GetSandboxManager();

		FLoadSandboxResult Result = SandboxManager.LoadSandbox(FLoadSandboxByDirectoryArgs{ InRootDirectory });
		if (Result.HasValue())
		{
			return true;
		}

		// Handle specific error cases (UE-356522)
		if (Result.HasError())
		{
			const FLoadSandboxError& Error = Result.GetError();
			if (Error.Reason == ELoadSandboxLoadErrorReason::IncompatibleVersion)
			{
				// Get sandbox name and version info for the error message
				const FString SandboxName = FPaths::GetPathLeaf(InRootDirectory);
				const FFileSandboxCore_VersionInfo SandboxVersion = LoadVersionInfo(InRootDirectory);

				FFileSandboxCore_EngineVersionInfo CurrentVersion;
				CurrentVersion.Initialize(FEngineVersion::Current());

				ShowIncompatibleVersionError(
					SandboxName,
					SandboxVersion.EngineVersion.ToString(),
					CurrentVersion.ToString()
				);
				return false;
			}
		}

		// Fall through to generic failure for other errors
	}
	return false;
}

bool FSandboxSystemModel::CanLoadSandbox(const FString& InRootDirectory)
{
	if (GEditor && GEditor->IsPlaySessionInProgress())
	{
		return false;
	}
	return !IsActiveSandbox(InRootDirectory) && !HasLeaveLock();
}

bool FSandboxSystemModel::IsActiveSandbox(const FString& InRootDirectory) const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	return Instance && FPaths::IsSamePath(Instance->GetRootDirectory(), InRootDirectory);
}

void FSandboxSystemModel::LeaveSandbox() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	SandboxManager.LeaveSandbox();
}

#define SET_REASON(ReasonText) if (OutReason){ *OutReason = ReasonText; }

bool FSandboxSystemModel::HasLeaveLock(FText* OutReason) const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	
	ISandboxLock* Lock = SandboxManager.GetActiveLock();
	return Lock && !Lock->CanLeaveSandbox(OutReason);
}
	
bool FSandboxSystemModel::IsAllowedToLeaveSandbox(FText* OutReason) const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();

	if (SandboxManager.GetActiveSandboxInstance() == nullptr)
	{
		SET_REASON(LOCTEXT("LeaveReason.NoSandbox", "No sandbox is active."));
		return false;
	}

	if (GEditor && GEditor->IsPlaySessionInProgress())
	{
		SET_REASON(LOCTEXT("LeaveReason.PlaySessionActive", "Cannot leave sandbox while a play session is active."));
		return false;
	}

	if (HasLeaveLock(OutReason))
	{
		SET_REASON(LOCTEXT("LeaveReason.Locked", "The engine is locked to be sandboxed."));
		return false;
	}

	const ELeaveSandboxErrorCode LeaveReason = SandboxManager.CanLeaveSandboxWithReason();
	return LeaveReason == ELeaveSandboxErrorCode::Success;
}
#undef SET_REASON

bool FSandboxSystemModel::CanLeaveSandboxWithoutFurtherActions() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	
	const bool bHasSandboxedChanges = Instance && !Instance->HasFileChanges();
	const bool bHasInMemoryChanges = GetInMemoryChanges().IsEmpty();
	return IsAllowedToLeaveSandbox() && bHasSandboxedChanges && bHasInMemoryChanges;
}

TArray<UPackage*> FSandboxSystemModel::GetInMemoryChanges() const
{
	return FileSandboxCore::GetDirtyPackages();
}

bool FSandboxSystemModel::RevertDirtyPackages()
{
	TArray<UPackage*> DirtyPackages = FileSandboxCore::GetDirtyPackages();
	if (DirtyPackages.IsEmpty())
	{
		return true;
	}

	TArray<UPackage*> PackagesToReload;
	TArray<FName> PackagesToPurge;
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	for (UPackage* Package : DirtyPackages)
	{
		if (FPackageName::DoesPackageExist(Package->GetName()))
		{
			PackagesToReload.Add(Package);
		}
		else
		{
			PackagesToPurge.Add(Package->GetFName());
			if (AssetRegistry)
			{
				ForEachObjectWithPackage(Package, [AssetRegistry](UObject* Object)
				{
					if (Object->IsAsset())
					{
						AssetRegistry->AssetDeleted(Object);
					}
					return true;
				});
			}
		}
	}

	FileSandboxCore::PurgePackages(PackagesToPurge);
	return FileSandboxCore::HotReloadPackages(PackagesToReload);
}

bool FSandboxSystemModel::SaveDirtyPackages()
{
	return HasActiveSandbox() && AssetViewUtils::SaveDirtyPackages();
}

void FSandboxSystemModel::DeleteSandbox(const FString& InSandboxRoot) const
{
	if (CanDeleteSandbox(InSandboxRoot))
	{
		using namespace FileSandboxCore;
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		ISandboxManager& SandboxManager = Module.GetSandboxManager();
		SandboxManager.DeleteSandbox(FDeleteSandboxByDirectoryArgs{ InSandboxRoot });
		NotifyIfBypassingRepository();
	}
}

#define SET_REASON(ReasonText) if (OutReason){ *OutReason = ReasonText; }
bool FSandboxSystemModel::CanDeleteSandbox(const FString& InSandboxRoot, FText* OutReason) const
{
	if (IsActiveSandbox(InSandboxRoot))
	{
		SET_REASON(LOCTEXT("Delete.ActiveSandbox", "Cannot delete active sandbox"));
		return false;
	}
	
	if (!FileSandboxCore::IsRootSandboxDirectory(InSandboxRoot))
	{
		SET_REASON(LOCTEXT("Delete.NotSandbox", "Directory does not contain any sandbox"));
		return false;
	}
	
	return true;
}
#undef SET_REASON

void FSandboxSystemModel::PersistAllChanges() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	
	if (CanPersistAllChanges() && ensure(Instance))
	{
		Instance->PersistAll();
	}
}

void FSandboxSystemModel::PersistFiles(TConstArrayView<FString> InFiles, FileSandboxCore::IPersistFeedback* InFeedback) const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	if (ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance())
	{
		const FPersistArgs Args 
		{
			.Files = InFiles,
			.Feedback = InFeedback
		};
		Instance->PersistSandbox(Args);
	}
}

bool FSandboxSystemModel::CanPersistAllChanges() const
{
	return HasActiveSandbox();
}

FileSandboxCore::FGatheredFileChanges FSandboxSystemModel::GatherActiveFileChanges() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	return Instance ? Instance->GatherChangedFiles() : FGatheredFileChanges{};
}

FileSandboxCore::FGatheredFileChanges FSandboxSystemModel::GatherFileChanges(const FString& InSandbox) const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	
	FGatheredFileChanges Result;
	SandboxManager.EnumerateFileChanges(InSandbox, [&Result](const FSandboxedFileChangeInfo& InChange)
	{
		Result.NonSandboxPaths.Add(InChange.Path);
		Result.FileActions.Add(InChange.Action);
		Result.Timestamps.Add(InChange.Timestamp);
		return FileSandboxCore::EBreakBehavior::Continue;
	}, EFileEnumerationFlags::IncludeTimestamps);
	return Result;
}

void FSandboxSystemModel::EnumerateFileChanges(
	const FString& InSandbox,
	TFunctionRef<FileSandboxCore::EBreakBehavior(const FileSandboxCore::FSandboxedFileChangeInfo& InChange)> InCallback
	)
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	SandboxManager.EnumerateFileChanges(InSandbox, InCallback);
}

void FSandboxSystemModel::RevertAllChanges() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	
	if (CanRevertAllChanges() && ensure(Instance))
	{
		Instance->RevertAll();
	}
}

bool FSandboxSystemModel::CanRevertAllChanges() const
{
	return HasActiveSandbox();
}

FString FSandboxSystemModel::GetActiveSandboxPath() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxInstance* SandboxInstance = Module.GetSandboxManager().GetActiveSandboxInstance();
	return SandboxInstance ? SandboxInstance->GetRootDirectory() : FString();
}

FString FSandboxSystemModel::GetActiveSandboxName() const
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxInstance* Instance = Module.GetSandboxManager().GetActiveSandboxInstance();
	return Instance && CanBeShownBySandboxedEditing(Instance->GetInitialMetaData()) ? Instance->GetInitialMetaData().Name : FString();
}

bool FSandboxSystemModel::HasActiveSandbox() const
{
	return !GetActiveSandboxName().IsEmpty();
}

void FSandboxSystemModel::ForEachSandbox(TFunctionRef<void(const FSandboxInfo& InRoot)> InProcess) const
{
	using namespace FileSandboxCore;

	if (IsUsingDefaultRepository())
	{
		// Use repository for default location (benefits from caching and directory watching)
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		ISandboxRepository& Repository = Module.GetDefaultSandboxRepository();
		Repository.ForEachSandbox([&InProcess](const FString& InRootPath, const FSandboxMetaInfo& MetaInfo)
		{
			const FFileSandboxCore_SandboxMetaData& MetaData = MetaInfo.UserMetaData;
			if (CanBeShownBySandboxedEditing(MetaData))
			{
				InProcess(FSandboxInfo(MetaData.Name, MetaData.Description, InRootPath, GetLastModified(InRootPath), MetaInfo.VersionInfo.EngineVersion));
			}
			return FileSandboxCore::EBreakBehavior::Continue;
		});
	}
	else
	{
		// Bypass repository and enumerate custom directory directly
		const FString CustomBaseDir = SandboxModel::GetBaseSandboxDirectory();
		FileSandboxCore::ForEachSandbox([&InProcess](const FString& InRootPath)
		{
			const TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InRootPath);
			if (MetaData && CanBeShownBySandboxedEditing(*MetaData))
			{
				const FFileSandboxCore_VersionInfo VersionInfo = LoadVersionInfo(InRootPath);
				InProcess(FSandboxInfo(MetaData->Name, MetaData->Description, InRootPath, GetLastModified(InRootPath), VersionInfo.EngineVersion));
			}
			return FileSandboxCore::EBreakBehavior::Continue;
		}, CustomBaseDir);
	}
}

TArray<FSandboxInfo> FSandboxSystemModel::GetKnownSandboxes() const
{
	TArray<FSandboxInfo> Result;
	ForEachSandbox([&Result](const FSandboxInfo& Info) { Result.Add(Info); });
	return Result;
}

TOptional<FSandboxInfo> FSandboxSystemModel::GetSandboxInfo(const FString& InSandboxRoot) const
{
	using namespace FileSandboxCore;

	TOptional<FSandboxInfo> Result;

	if (IsUsingDefaultRepository())
	{
		// Use repository for default location
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		ISandboxRepository& Repository = Module.GetDefaultSandboxRepository();
		Repository.ReadMetaData(InSandboxRoot, [&InSandboxRoot, &Result](const FSandboxMetaInfo& MetaInfo)
		{
			const FFileSandboxCore_SandboxMetaData& MetaData = MetaInfo.UserMetaData;
			if (CanBeShownBySandboxedEditing(MetaData))
			{
				Result.Emplace(MetaData.Name, MetaData.Description, InSandboxRoot, MetaInfo.LastModified, MetaInfo.VersionInfo.EngineVersion);
			}
		});
	}
	else
	{
		// Bypass repository and load directly from custom directory
		const TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InSandboxRoot);
		if (MetaData && CanBeShownBySandboxedEditing(*MetaData))
		{
			const FFileSandboxCore_VersionInfo VersionInfo = LoadVersionInfo(InSandboxRoot);
			Result.Emplace(MetaData->Name, MetaData->Description, InSandboxRoot, GetLastModified(InSandboxRoot), VersionInfo.EngineVersion);
		}
	}

	return Result;
}

void FSandboxSystemModel::SetDescription(const FString& InSandboxRoot, const FString& InDescription) const
{
	using namespace FileSandboxCore;

	if (IsUsingDefaultRepository())
	{
		// Use repository for default location (repository will fire events)
		IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
		ISandboxRepository& Repository = Module.GetDefaultSandboxRepository();
		Repository.ReadMetaData(InSandboxRoot, [&InSandboxRoot, &InDescription](const FSandboxMetaInfo& MetaInfo)
		{
			if (CanBeShownBySandboxedEditing(MetaInfo.UserMetaData))
			{
				FFileSandboxCore_SandboxMetaData MetaData = MetaInfo.UserMetaData;
				MetaData.Description = InDescription;
				SaveMetaData(MetaData, InSandboxRoot);
			}
		});
	}
	else
	{
		// Bypass repository and load/save directly from custom directory
		const TOptional<FFileSandboxCore_SandboxMetaData> MetaData = LoadMetaData(InSandboxRoot);
		if (MetaData && CanBeShownBySandboxedEditing(*MetaData))
		{
			FFileSandboxCore_SandboxMetaData UpdatedMetaData = *MetaData;
			UpdatedMetaData.Description = InDescription;
			SaveMetaData(UpdatedMetaData, InSandboxRoot);
			NotifyIfBypassingRepository();
		}
	}
}

bool FSandboxSystemModel::RenameSandbox(const FString& InSandboxRoot, const FString& InNewName) const
{
	const FString OldSandboxName = FPaths::GetBaseFilename(InSandboxRoot);
	const FString BaseDirectory = FPaths::GetPath(InSandboxRoot);
	const bool bSuccess = FileSandboxCore::RenameSandboxWithDirectory(InNewName, OldSandboxName, BaseDirectory);

	if (bSuccess)
	{
		NotifyIfBypassingRepository();
	}

	return bSuccess;
}

#define SET_ERROR(Text) if (OutError){ *OutError = Text; }
bool FSandboxSystemModel::CanRenameSandbox(const FString& InSandboxRoot, const FString& InNewName, FText* OutError) const
{
	using namespace FileSandboxCore;
	const FString OldSandboxName = FPaths::GetBaseFilename(InSandboxRoot);
	const FString BaseDirectory = FPaths::GetPath(InSandboxRoot);
	
	switch (CanRenameSandboxWithDirectory(InNewName, OldSandboxName, BaseDirectory))
	{
	case ESandboxRenameSuitability::Allowed: return true;
		
	case ESandboxRenameSuitability::SandboxDoesNotExist:
		ensure(false); // UI must have gotten out of sync. Investigate.
		SET_ERROR(LOCTEXT("SandboxDoesNotExist", "Sandbox does not exist anymore.")); 
		break;
	case ESandboxRenameSuitability::ActiveSandbox:
		SET_ERROR(LOCTEXT("ActiveSandbox.SpecifiedName", "You cannot rename the active sandbox.")); 
		break;
	case ESandboxRenameSuitability::NameIsSame:
		SET_ERROR(LOCTEXT("ActiveSandbox.NameIsSame", "The new name must be different from the current name.")); 
		break;
	case ESandboxRenameSuitability::InvalidDirectory:
		SET_ERROR(LOCTEXT("InvalidDirectory", "The name contains characters that aren’t allowed in directory names.")); 
		break;
	case ESandboxRenameSuitability::EmptyName:
		SET_ERROR(LOCTEXT("EmptyName", "Name cannot be empty.")); 
		break;
	default:
		checkNoEntry();
	}
		
	return false;
}
#undef SET_ERROR

#define SET_ERROR(Text) if (OutError){ *OutError = Text; }
bool FSandboxSystemModel::IsAllowedToRenameSandbox(const FString& InSandboxRoot, FText* OutError) const
{
	const bool bCanRename = FileSandboxCore::IsAllowedToRenameSandbox(InSandboxRoot);
	if (!bCanRename)
	{
		SET_ERROR(LOCTEXT("ActiveSandbox.UnspecifiedName", "Active sandbox cannot be renamed."))
	}
	return bCanRename;
}
#undef SET_ERROR

void FSandboxSystemModel::HandleLoadSandbox(FileSandboxCore::ISandboxInstance& InInstance)
{
	OnLoadSandboxDelegate.Broadcast();
	InInstance.OnSandboxedFilesChanged().AddRaw(this, &FSandboxSystemModel::HandleActiveSandboxFilesChanged);
}

void FSandboxSystemModel::HandleLeaveSandbox()
{
	OnLeaveSandboxDelegate.Broadcast();
}

void FSandboxSystemModel::HandleActiveSandboxFilesChanged()
{
	OnSandboxFilesChangedDelegate.Broadcast();
}

bool FSandboxSystemModel::IsUsingDefaultRepository() const
{
	const FString CustomBaseDir = SandboxModel::GetBaseSandboxDirectory();
	const FString DefaultBaseDir = FileSandboxCore::GetBaseSandboxDirectory();
	return CustomBaseDir == DefaultBaseDir;
}

void FSandboxSystemModel::NotifyIfBypassingRepository() const
{
	if (!IsUsingDefaultRepository())
	{
		OnKnownSandboxesChangedDelegate.Broadcast();
	}
}

namespace SandboxModel
{
TOptional<FString> GetSandboxPathFor(const FString& InSandbox, const FString& InNonSandboxPath)
{
	return FileSandboxCore::GetSandboxPathFor(InSandbox, InNonSandboxPath);
}

TOptional<FString> GetSandboxName(const FString& InSandboxDirectory)
{
	return FileSandboxCore::GetSandboxName(InSandboxDirectory);
}

FString GetBaseSandboxDirectory()
{
	// Priority: Command line > Custom setting > Default
	// Check if settings specify a custom directory
	const USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();
	if (Settings && !Settings->CustomSandboxStorageDirectory.Path.IsEmpty())
	{
		// FileSandboxCore::GetBaseSandboxDirectory() handles command line override internally
		// We need to ensure command line still wins, so check it first
		const FString CoreResult = FileSandboxCore::GetBaseSandboxDirectory();
		const FString DefaultIntermediate = FPaths::ProjectIntermediateDir() / TEXT("Sandboxes");

		// If CoreResult != default, it means command line was used, so respect that
		if (CoreResult != DefaultIntermediate)
		{
			return CoreResult;
		}

		// Otherwise use custom setting
		return Settings->CustomSandboxStorageDirectory.Path;
	}

	// Fall back to FileSandboxCore (handles command line + default)
	return FileSandboxCore::GetBaseSandboxDirectory();
}

bool HasActiveSandbox()
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxInstance* Instance = Module.GetSandboxManager().GetActiveSandboxInstance();
	return Instance != nullptr;
}

void RevertSpecifiedChanges(TConstArrayView<FString> InFiles)
{
	using namespace FileSandboxCore;
	IFileSandboxCoreModule& Module = IFileSandboxCoreModule::Get();
	ISandboxManager& SandboxManager = Module.GetSandboxManager();
	ISandboxInstance* Instance = SandboxManager.GetActiveSandboxInstance();
	
	if (CanRevertSpecifiedChanges(InFiles) && ensure(Instance))
	{
		Instance->RevertSpecified(InFiles);
	}
}

bool CanRevertSpecifiedChanges(TConstArrayView<FString> InFiles)
{
	return HasActiveSandbox() && !InFiles.IsEmpty();
}
}
}

#undef LOCTEXT_NAMESPACE
