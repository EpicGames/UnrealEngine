// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"
#include "Types/RepositoryChangedEvent.h"

namespace UE::FileSandboxCore
{
class IPersistFeedback;
class ISandboxInstance;
enum class EBreakBehavior : uint8;
enum class ESandboxFileChange : uint8;
struct FGatheredFileChanges;
struct FSandboxedFileChangeInfo;
}

template<typename OptionalType> struct TOptional;

namespace UE::SandboxedEditing
{
struct FCreateSandboxArgs;
struct FSandboxInfo;
	
/**
 * Model for creating, loading, and querying the active sandbox.
 * 
 * Bridges the sandbox system without exposing it directly.
 * Members added here should have direct corresponding UI elements.
 */
class FSandboxSystemModel : public FNoncopyable
{
public:
	
	FSandboxSystemModel();
	~FSandboxSystemModel();

	/** Creates a new sandbox with the given name. */
	bool CreateNewSandbox(const FCreateSandboxArgs& InArgs);
	/** @return Whether it is valid to call CreateNewSandbox. */
	bool CanCreateNewSandbox(const FString& InName, FText* Reason = nullptr);
	/** @return Whether we can create a new sandbox. This is allowed as long as there is no leave lock. */
	bool CanCreateNewSandbox() const;
		
	/** Loads the specified sandbox. */
	bool LoadSandbox(const FString& InRootDirectory);
	/** @return Whether the user can join this sandbox (i.e. not yet in this sandbox) */
	bool CanLoadSandbox(const FString& InRootDirectory);
	/** @return Whether this is the active sandbox. */
	bool IsActiveSandbox(const FString& InRootDirectory) const;

	/** Leaves the current sandbox. */
	void LeaveSandbox() const;
	/** @return Whether the sandbox can be left (false if there is no active one). */
	bool IsAllowedToLeaveSandbox() const { return IsAllowedToLeaveSandbox(nullptr); }
	/** @return Whether the sandbox can be left (false if there is no active one). */
	bool IsAllowedToLeaveSandbox(FText* OutReason) const;
	/** @return Whether the engine is in a valid state to leave the sandbox, i.e. no other clean up actions need to be performed. */
	bool CanLeaveSandboxWithoutFurtherActions() const;
	
	/** @return The in-memory changes preventing the sandbox from being left. */
	TArray<UPackage*> GetInMemoryChanges() const;
	/** Reverts in-memory changes on the given packages, and performs related actions, such as reloading the underlying asset editors, etc. */
	bool RevertDirtyPackages();
	/** Saves the in-memory changes of the given packages, so the packages are no longer dirty. */
	bool SaveDirtyPackages();
	
	/** Deletes the sandbox. */
	void DeleteSandbox(const FString& InSandboxRoot) const;
	/** @return Whether the sandbox can be deleted. */
	bool CanDeleteSandbox(const FString& InSandboxRoot, FText* OutReason = nullptr) const;

	/** Persists all changes */
	void PersistAllChanges() const;
	/** Persists given files. */
	void PersistFiles(TConstArrayView<FString> InFiles, FileSandboxCore::IPersistFeedback* InFeedback = nullptr) const;
	/** @return Whether all changes can be persisted */
	bool CanPersistAllChanges() const;
	
	/** @return The file changes that were performed by the sandbox. */
	FileSandboxCore::FGatheredFileChanges GatherActiveFileChanges() const;
	/** @return The file changes of the specified sandbox. */
	FileSandboxCore::FGatheredFileChanges GatherFileChanges(const FString& InSandbox) const;
	
	/** Invokes InCallback for each file change. */
	void EnumerateFileChanges(
		const FString& InSandbox, 
		TFunctionRef<FileSandboxCore::EBreakBehavior(const FileSandboxCore::FSandboxedFileChangeInfo& InChange)> InCallback
		);

	/** Reverts all changes */
	void RevertAllChanges() const;
	/** @return Whether all changes can be reverted */
	bool CanRevertAllChanges() const; 

	/** @return Path to the root directory of the active sandbox. Empty if none is active. */
	FString GetActiveSandboxPath() const;
	/** @return Name of the currently active sandbox. Empty if none active. */
	FString GetActiveSandboxName() const;
	/** @return Whether there is an active sandbox */
	bool HasActiveSandbox() const;
	
	/** Enumerates all known sandboxes. */
	void ForEachSandbox(TFunctionRef<void(const FSandboxInfo& InRoot)> InProcess) const;
	/** @return The sandboxes we know of. */
	TArray<FSandboxInfo> GetKnownSandboxes() const;
	
	/** @return Info about the sandbox. Unset if sandbox does not exist. */
	TOptional<FSandboxInfo> GetSandboxInfo(const FString& InSandboxRoot) const;
	/** Sets the sandbox description. */
	void SetDescription(const FString& InSandboxRoot, const FString& InDescription) const;
	
	/** Tries to rename InSandboxRoot to InNewName. */
	bool RenameSandbox(const FString& InSandboxRoot, const FString& InNewName) const;
	/** @returnWhether the sandbox at InSandboxRoot can be renamed to InNewName. */
	bool CanRenameSandbox(const FString& InSandboxRoot, const FString& InNewName, FText* OutError = nullptr) const;
	/** @return Whether it is allowd to rename the InSandboxRoot, i.e. it is not active. */
	bool IsAllowedToRenameSandbox(const FString& InSandboxRoot, FText* OutError = nullptr) const;
	
	/** Invoked when the result of GetKnownSandboxes may have changed. */
	FSimpleMulticastDelegate& OnKnownSandboxesChanged() { return OnKnownSandboxesChangedDelegate; }
	
	/** Invoked when the a new sandbox is loaded. */
	FSimpleMulticastDelegate& OnLoadSandbox() { return OnLoadSandboxDelegate; }
	/** Invoked when the current sandbox is left. */
	FSimpleMulticastDelegate& OnLeaveSandbox() { return OnLeaveSandboxDelegate; }
	
	/** Invoked when the files in the active sandbox have changed. */
	FSimpleMulticastDelegate& OnSandboxFilesChanged() { return OnSandboxFilesChangedDelegate; }

private:
	/** @return true if a leave lock is currently registered. */
	bool HasLeaveLock(FText* OutReason = nullptr) const;

	/** @return true if using the default repository (not a custom directory). */
	bool IsUsingDefaultRepository() const;

	/** Fires OnKnownSandboxesChanged event if using a custom directory (bypassing repository's directory watching). */
	void NotifyIfBypassingRepository() const;
		
	/** Invoked when the result of GetKnownSandboxes may have changed. */
	FSimpleMulticastDelegate OnKnownSandboxesChangedDelegate;
	
	/** Invoked when the a new sandbox is loaded. */
	FSimpleMulticastDelegate OnLoadSandboxDelegate;
	/** Invoked when the current sandbox is left. */
	FSimpleMulticastDelegate OnLeaveSandboxDelegate;
	
	/** Invoked when the files in the active sandbox have changed. */
	FSimpleMulticastDelegate OnSandboxFilesChangedDelegate;
	
	void HandleSandboxesChanged(const FileSandboxCore::FRepositoryChangedEvent&) const { OnKnownSandboxesChangedDelegate.Broadcast(); }
	void HandleMetaDataChanged(const FString&) const { OnKnownSandboxesChangedDelegate.Broadcast(); }
	void HandleCustomDirectoryChanged() const { OnKnownSandboxesChangedDelegate.Broadcast(); }

	void HandleLoadSandbox(FileSandboxCore::ISandboxInstance& InInstance);
	void HandleLeaveSandbox();
	/** Invoked when the set of files touched by the active sandbox changes. */
	void HandleActiveSandboxFilesChanged();
};

/** 
 * Stateless functions that interact with the sandbox system.
 * This is an abstraction to protect the UI code from the underlying FileSandboxCore types.
 * TODO: Move stateless functions from FSandboxSystemModel into here.
 */
namespace SandboxModel
{
/** @return Gets the sandbox path for InNonSandboxPath. */
TOptional<FString> GetSandboxPathFor(const FString& InSandbox, const FString& InNonSandboxPath);

/** @return Name of the sandbox if the directory is the root of a sandbox. Unset otherwise. */
TOptional<FString> GetSandboxName(const FString& InSandboxDirectory);

/** @return The base directory in which sandboxes are placed. */
FString GetBaseSandboxDirectory();

/** @return Whether there is an active sandbox */
bool HasActiveSandbox();

/** Reverts the changes of the specified files. */
void RevertSpecifiedChanges(TConstArrayView<FString> InFiles);
/** @return Whether the specified files can be reverted. */
bool CanRevertSpecifiedChanges(TConstArrayView<FString> InFiles);
}
}