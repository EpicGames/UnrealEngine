// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxTagging.h"

#include "Data/SandboxMetaData.h"
#include "Utils/BuiltInTags.h"

namespace UE::SandboxedEditing
{
bool IsOwnedBySandboxedEditing(const FFileSandboxCore_SandboxMetaData& InMetaData)
{
	return InMetaData.Tags.Contains(SandboxedEditingTag.Resolve());
}

bool CanBeShownBySandboxedEditing(const FFileSandboxCore_SandboxMetaData& InMetaData)
{
	return IsOwnedBySandboxedEditing(InMetaData) 
		|| InMetaData.Tags.Contains(FileSandboxCore::Tag_CommandLineSandbox.Resolve());
}

void MarkOwnedBySandboxedEditing(FFileSandboxCore_SandboxMetaData& InMetaData)
{
	InMetaData.Tags.Add(SandboxedEditingTag.Resolve());
}
}
