// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

struct FFileSandboxCore_SandboxMetaData;

namespace UE::SandboxedEditing
{
/** Tag added to sandboxes created by the Sandboxed Editing plugin. */
constexpr FLazyName SandboxedEditingTag("SandboxedEditingTag");

/** @return Whether the metadata is marked to belong to the Sandboxed Editing application. */
bool IsOwnedBySandboxedEditing(const FFileSandboxCore_SandboxMetaData& InMetaData);

/** @return Whether the metadata indicates that Sandboxed Editing can interact with this sandbox. */
bool CanBeShownBySandboxedEditing(const FFileSandboxCore_SandboxMetaData& InMetaData);

/** Modifies the metadata such that we know it belongs to Sandboxed Editing application. */
void MarkOwnedBySandboxedEditing(FFileSandboxCore_SandboxMetaData& InMetaData);
}
