// Copyright Epic Games, Inc. All Rights Reserved.

#include "Notifications.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SandboxedEditing.Notifications"

namespace UE::SandboxedEditing
{
void ShowCreatedSandbox(const FString& InSandboxName)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}
	
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(
		FText::Format(LOCTEXT("JoinedFmt", "Joined sandbox {0}"), FText::FromString(InSandboxName))
		);
	Args.bFireAndForget = true;
	Args.ExpireDuration = 2.f;
	
	NotificationManager.AddNotification(Args);
}

void ShowLoadedSandbox(const FString& InSandboxName)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}
	
	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(
		FText::Format(LOCTEXT("LoadedFmt", "Loaded sandbox {0}"), FText::FromString(InSandboxName))
		);
	Args.bFireAndForget = true;
	Args.ExpireDuration = 2.f;
	
	NotificationManager.AddNotification(Args);
}

void ShowFailedToLoadSandbox(const FString& InSandboxName)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(
		FText::Format(LOCTEXT("FailedToLoadFmt.Title", "Failed to load sandbox {0}"), FText::FromString(InSandboxName))
		);
	Args.bFireAndForget = true;
	Args.ExpireDuration = 2.f;

	const TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Args);
	Item->SetCompletionState(SNotificationItem::CS_Fail);
	Item->SetSubText(LOCTEXT("FailedToLoadFmt.Description", "See log for more info."));
}

void ShowIncompatibleVersionError(const FString& InSandboxName, const FString& InSandboxVersion, const FString& InCurrentVersion)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(
		FText::Format(LOCTEXT("IncompatibleVersionFmt.Title", "Cannot load sandbox {0}"), FText::FromString(InSandboxName))
		);
	Args.bFireAndForget = true;
	Args.ExpireDuration = 5.f;

	const TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Args);
	Item->SetCompletionState(SNotificationItem::CS_Fail);
	Item->SetSubText(
		FText::Format(
			LOCTEXT("IncompatibleVersionFmt.Description", "Sandbox was created with version {0}, but you are running version {1}. You cannot load a sandbox created with a newer version."),
			FText::FromString(InSandboxVersion),
			FText::FromString(InCurrentVersion)
		)
	);
}

void ShowLeftSandbox(const FString& InSandboxName)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(
		FText::Format(LOCTEXT("LeftFmt", "Left sandbox {0}"), FText::FromString(InSandboxName))
		);
	Args.bFireAndForget = true;
	Args.ExpireDuration = 2.f;

	NotificationManager.AddNotification(Args);
}

void ShowCannotLeaveDuringPlayMode()
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Args(LOCTEXT("CannotLeaveDuringPlay.Title", "Cannot leave sandbox during Play in Editor"));
	Args.bFireAndForget = true;
	Args.ExpireDuration = 3.f;

	const TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Args);
	Item->SetSubText(LOCTEXT("CannotLeaveDuringPlay.Description", "Please stop the current play session first."));
}

TSharedPtr<SNotificationItem> ShowInMemoryChangesWarning(const FString& InSandboxName, TFunction<void()> OnDiscard, TFunction<void()> OnCancel)
{
	if (!FSlateApplication::IsInitialized())
	{
		return nullptr;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();

	// Create a shared pointer to hold the weak notification so lambdas can capture it by value
	TSharedPtr<TWeakPtr<SNotificationItem>> WeakNotificationPtr = MakeShared<TWeakPtr<SNotificationItem>>();

	FNotificationInfo Info(
		FText::Format(LOCTEXT("LoadingSandbox", "Loading {0}"), FText::FromString(InSandboxName))
	);
	Info.SubText = LOCTEXT("InMemoryChanges.Body", "The project has in-memory changes. Continuing will discard changes.");
	Info.bFireAndForget = false;
	Info.ExpireDuration = 0.0f;
	Info.ButtonDetails =
	{
		FNotificationButtonInfo(
			LOCTEXT("Discard", "Discard"),
			LOCTEXT("Discard.ToolTip", "Discard in-memory changes and load sandbox"),
			FSimpleDelegate::CreateLambda([OnDiscard, WeakNotificationPtr]()
			{
				if (TSharedPtr<SNotificationItem> PinnedNotification = WeakNotificationPtr->Pin())
				{
					PinnedNotification->ExpireAndFadeout();
				}
				if (OnDiscard)
				{
					OnDiscard();
				}
			}),
			SNotificationItem::CS_None
		),
		FNotificationButtonInfo(
			LOCTEXT("Cancel", "Cancel"),
			LOCTEXT("Cancel.ToolTip", "Cancel loading sandbox"),
			FSimpleDelegate::CreateLambda([OnCancel, WeakNotificationPtr]()
			{
				if (TSharedPtr<SNotificationItem> PinnedNotification = WeakNotificationPtr->Pin())
				{
					PinnedNotification->ExpireAndFadeout();
				}
				if (OnCancel)
				{
					OnCancel();
				}
			}),
			SNotificationItem::CS_None
		)
	};

	TSharedPtr<SNotificationItem> Notification = NotificationManager.AddNotification(Info);
	*WeakNotificationPtr = Notification;
	return Notification;
}
}

#undef LOCTEXT_NAMESPACE
