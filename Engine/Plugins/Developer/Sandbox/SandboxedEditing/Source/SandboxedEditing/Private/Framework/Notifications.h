// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class FString;
class SNotificationItem;

namespace UE::SandboxedEditing
{
/** Shows a notification that the user joined a sandbox. */
void ShowCreatedSandbox(const FString& InSandboxName);

/** Shows the sandbox was loaded successfully. */
void ShowLoadedSandbox(const FString& InSandboxName);
/** Show that the sandbox could not be loaded. */
void ShowFailedToLoadSandbox(const FString& InSandboxName);
/** Show that the sandbox could not be loaded due to incompatible version. */
void ShowIncompatibleVersionError(const FString& InSandboxName, const FString& InSandboxVersion, const FString& InCurrentVersion);

/** Shows notification that the user left a sandbox. */
void ShowLeftSandbox(const FString& InSandboxName);

/** Shows notification that the user cannot leave the sandbox while a play session is active. */
void ShowCannotLeaveDuringPlayMode();

/**
 * Shows a notification warning the user about in-memory changes before loading a sandbox.
 * @param InSandboxName The name of the sandbox being loaded
 * @param OnDiscard Callback invoked when user chooses to discard changes
 * @param OnCancel Callback invoked when user chooses to cancel
 * @return The notification item.
 */
TSharedPtr<SNotificationItem> ShowInMemoryChangesWarning(const FString& InSandboxName, TFunction<void()> OnDiscard, TFunction<void()> OnCancel);
}

