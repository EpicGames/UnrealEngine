// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxedEditingApp.h"

#include "EntryPoint/SandboxedEditingEntryPoint.h"
#include "EntryPoint/SandboxedEditingSandboxIntegration.h"
#include "Models/SandboxSystemModel.h"
#include "StartupBehavior/StartupBehaviorHandler.h"

namespace UE::SandboxedEditing
{
FSandboxedEditingApp::FSandboxedEditingApp()
	: SandboxModel(MakeShared<FSandboxSystemModel>())
	, SandboxIntegration(MakeShared<FSandboxedEditingSandboxIntegration>())
	, StatusBarFeature(SandboxModel)
	, BrowserFeature(SandboxModel, SandboxIntegration->GetExternalSandboxViewModel())
	, ContentBrowserBadgeFeature(SandboxModel)
	, StartupBehaviorHandler(SandboxModel)
	, ShutdownCleanupHandler(SandboxModel)
{
	SandboxIntegration->GetEntryPoint()->OnRequestSummonUI().AddLambda([this] { BrowserFeature.SummonUI(); });
}
}
