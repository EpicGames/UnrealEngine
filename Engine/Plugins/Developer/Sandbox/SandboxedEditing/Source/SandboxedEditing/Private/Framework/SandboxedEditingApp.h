// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/Browser/BrowserFeature.h"
#include "Features/ContentBrowserBadge/ContentBrowserBadgeFeature.h"
#include "Features/StatusBar/StatusBarFeature.h"
#include "Framework/Cleanup/ShutdownCleanupHandler.h"
#include "Framework/StartupBehavior/StartupBehaviorHandler.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;
class FSandboxedEditingSandboxIntegration;
class FStartupBehaviorHandler;

/**
 * Creates all relevant systems to deliver the sandboxed editing experience in the editor.
 * This is a coordinator: It sets up features, which are isolated areas in the editor that interact with sandboxing.
 */
class FSandboxedEditingApp : public FNoncopyable
{
public:
	FSandboxedEditingApp();

private:

	/**
	 * View model for interacting with sandbox system.
	 * Shared pointer in case the UI to avoid crash in case that though some bug the UI outlives this app for whatever reason.
	 */
	const TSharedRef<FSandboxSystemModel> SandboxModel;

	/** Responsible for registering Sandboxed Editing with the sandbox system. */
	const TSharedRef<FSandboxedEditingSandboxIntegration> SandboxIntegration;

	/** Extends the status bar for Sandboxed Editing. */
	const FStatusBarFeature StatusBarFeature;
	/** Extends the editor with a nomad tab containing a browser for sandboxes. */
	const FBrowserFeature BrowserFeature;
	/** Displays badges on Content Browser items for files modified in the active sandbox. */
	const FContentBrowserBadgeFeature ContentBrowserBadgeFeature;
	/** Handles startup behavior (creating/loading sandboxes on editor startup). */
	const FStartupBehaviorHandler StartupBehaviorHandler;
	/** Handles cleanup of empty sandboxes on editor shutdown. */
	const FShutdownCleanupHandler ShutdownCleanupHandler;
};
}

