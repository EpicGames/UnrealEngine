// Copyright Epic Games, Inc. All Rights Reserved.

#include "StartupBehaviorHandler.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Models/CreateSandboxArgs.h"
#include "Framework/Models/SandboxInfo.h"
#include "Framework/Models/SandboxSystemModel.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LogSandboxedEditing.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/DateTime.h"
#include "Misc/Parse.h"
#include "NamingTokensEngineSubsystem.h"
#include "SandboxedEditingSettings.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "SandboxedEditing.StartupBehavior"

namespace UE::SandboxedEditing
{

FStartupBehaviorHandler::FStartupBehaviorHandler(const TSharedRef<FSandboxSystemModel>& InModel)
	: SandboxModel(InModel)
{
	// Register for editor initialization - this is when we execute startup behavior
	FEditorDelegates::OnEditorInitialized.AddRaw(this, &FStartupBehaviorHandler::OnEditorInitialized);

	// Track sandbox loads for last used tracking
	LoadSandboxHandle = SandboxModel->OnLoadSandbox().AddRaw(this, &FStartupBehaviorHandler::OnSandboxLoaded);

	UE_LOGF(LogSandboxedEditing, Verbose, "StartupBehaviorHandler initialized");
}

FStartupBehaviorHandler::~FStartupBehaviorHandler()
{
	// Dismiss any active notification and disable buttons to prevent callbacks during fade
	if (CurrentNotification.IsValid())
	{
		// SetEnabled(false) prevents button clicks during fade animation, ensuring callbacks cannot fire
		CurrentNotification->SetEnabled(false);
		CurrentNotification->SetCompletionState(SNotificationItem::CS_None);
		CurrentNotification->ExpireAndFadeout();
		CurrentNotification.Reset();
	}

	FEditorDelegates::OnEditorInitialized.RemoveAll(this);

	if (LoadSandboxHandle.IsValid())
	{
		SandboxModel->OnLoadSandbox().Remove(LoadSandboxHandle);
	}

	UE_LOGF(LogSandboxedEditing, Verbose, "StartupBehaviorHandler destroyed");
}

void FStartupBehaviorHandler::OnEditorInitialized(double Duration)
{
	UE_LOGF(LogSandboxedEditing, Log, "Editor initialized in %.2f seconds, checking sandbox startup behavior", Duration);

	// Check if another plugin already started a sandbox
	if (SandboxModel->HasActiveSandbox())
	{
		UE_LOGF(LogSandboxedEditing, Log, "Sandbox already active (%ls), skipping startup behavior",
			*SandboxModel->GetActiveSandboxName());
		return;
	}

	ExecuteStartupBehavior();
}

bool FStartupBehaviorHandler::TryGetStartupBehaviorFromCommandLine(ESandboxStartupBehavior& OutBehavior) const
{
	FString BehaviorString;
	if (!FParse::Value(FCommandLine::Get(), TEXT("SANDBOXSTARTUP="), BehaviorString))
	{
		return false;
	}

	BehaviorString.TrimStartAndEndInline();

	if (BehaviorString.Equals(TEXT("DoNothing"), ESearchCase::IgnoreCase))
	{
		OutBehavior = ESandboxStartupBehavior::DoNothing;
		return true;
	}
	else if (BehaviorString.Equals(TEXT("PromptToJoin"), ESearchCase::IgnoreCase))
	{
		OutBehavior = ESandboxStartupBehavior::PromptToJoin;
		return true;
	}
	else if (BehaviorString.Equals(TEXT("CreateNewSandbox"), ESearchCase::IgnoreCase))
	{
		OutBehavior = ESandboxStartupBehavior::CreateNewSandbox;
		return true;
	}
	else if (BehaviorString.Equals(TEXT("JoinPreviousSandbox"), ESearchCase::IgnoreCase))
	{
		OutBehavior = ESandboxStartupBehavior::JoinPreviousSandbox;
		return true;
	}
	else
	{
		UE_LOGF(LogSandboxedEditing, Warning, "Unknown value for -SANDBOXSTARTUP: %ls", *BehaviorString);
		return false;
	}
}

bool FStartupBehaviorHandler::TryGetNameTemplateFromCommandLine(FString& OutTemplate) const
{
	if (FParse::Value(FCommandLine::Get(), TEXT("SANDBOXNAMETEMPLATE="), OutTemplate))
	{
		OutTemplate.TrimStartAndEndInline();
		return !OutTemplate.IsEmpty();
	}
	return false;
}

void FStartupBehaviorHandler::ExecuteStartupBehavior()
{
	const USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();
	if (!Settings)
	{
		UE_LOGF(LogSandboxedEditing, Warning, "Failed to get SandboxedEditingSettings");
		return;
	}

	// Check for command line overrides
	ESandboxStartupBehavior Behavior = Settings->StartupBehavior;
	if (TryGetStartupBehaviorFromCommandLine(Behavior))
	{
		UE_LOGF(LogSandboxedEditing, Log, "Using command line override for startup behavior: %d", static_cast<int32>(Behavior));
	}

	switch (Behavior)
	{
	case ESandboxStartupBehavior::DoNothing:
		HandleDoNothing();
		break;

	case ESandboxStartupBehavior::PromptToJoin:
		HandlePromptToJoin();
		break;

	case ESandboxStartupBehavior::CreateNewSandbox:
		HandleCreateNewSandbox();
		break;

	case ESandboxStartupBehavior::JoinPreviousSandbox:
		HandleJoinPreviousSandbox();
		break;

	default:
		UE_LOGF(LogSandboxedEditing, Warning, "Unknown startup behavior: %d", static_cast<int32>(Behavior));
		break;
	}
}

void FStartupBehaviorHandler::HandleDoNothing()
{
	UE_LOGF(LogSandboxedEditing, Verbose, "Startup behavior: Do Nothing");
	// Nothing to do - this is the default
}

void FStartupBehaviorHandler::HandlePromptToJoin()
{
	UE_LOGF(LogSandboxedEditing, Log, "Startup behavior: Prompt to Join");

	if (!FSlateApplication::IsInitialized())
	{
		UE_LOGF(LogSandboxedEditing, Warning, "Slate not initialized, cannot show prompt");
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();

	FNotificationInfo Info(LOCTEXT("Prompt.Title", "Join Sandbox?"));
	Info.SubText = LOCTEXT("Prompt.SubText", "Would you like to create a new sandbox or join your previous sandbox?");
	Info.bFireAndForget = false;
	Info.bUseLargeFont = false;
	Info.bUseSuccessFailIcons = false;
	Info.ButtonDetails = {
		FNotificationButtonInfo(
			LOCTEXT("Prompt.CreateNew", "Create New"),
			LOCTEXT("Prompt.CreateNew.TT", "Create a new sandbox"),
			FSimpleDelegate::CreateRaw(this, &FStartupBehaviorHandler::HandleCreateNewSandbox),
			SNotificationItem::CS_None
		),
		FNotificationButtonInfo(
			LOCTEXT("Prompt.JoinPrevious", "Join Previous"),
			LOCTEXT("Prompt.JoinPrevious.TT", "Load your most recently used sandbox"),
			FSimpleDelegate::CreateRaw(this, &FStartupBehaviorHandler::HandleJoinPreviousSandbox),
			SNotificationItem::CS_None
		),
		FNotificationButtonInfo(
			LOCTEXT("Prompt.Skip", "Skip"),
			LOCTEXT("Prompt.Skip.TT", "Continue without a sandbox"),
			FSimpleDelegate::CreateLambda([this]() {
				if (CurrentNotification.IsValid())
				{
					CurrentNotification->SetCompletionState(SNotificationItem::CS_None);
					CurrentNotification->ExpireAndFadeout();
					CurrentNotification.Reset();
				}
				UE_LOGF(LogSandboxedEditing, Log, "User chose to skip sandbox on startup");
			}),
			SNotificationItem::CS_None
		)
	};

	CurrentNotification = NotificationManager.AddNotification(Info);
}

void FStartupBehaviorHandler::HandleCreateNewSandbox()
{
	UE_LOGF(LogSandboxedEditing, Log, "Startup behavior: Create New Sandbox");

	// Dismiss any active notification
	if (CurrentNotification.IsValid())
	{
		CurrentNotification->ExpireAndFadeout();
		CurrentNotification.Reset();
	}

	const USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();
	if (!Settings)
	{
		ShowErrorNotification(LOCTEXT("Error.NoSettings", "Failed to read settings"));
		return;
	}

	// Check for command line override of name template
	FString NameTemplate;
	if (TryGetNameTemplateFromCommandLine(NameTemplate))
	{
		UE_LOGF(LogSandboxedEditing, Log, "Using command line override for name template: %ls", *NameTemplate);
	}
	else
	{
		NameTemplate = Settings->DefaultSandboxNameTemplate;
	}

	// Resolve naming tokens in template
	FString ResolvedBaseName = ResolveNamingTokens(NameTemplate);
	if (ResolvedBaseName.IsEmpty())
	{
		ResolvedBaseName = TEXT("Sandbox");
	}

	// Generate unique name (auto-increment if needed)
	FString UniqueName = GenerateUniqueSandboxName(ResolvedBaseName);
	UE_LOGF(LogSandboxedEditing, Log, "Creating new sandbox with name: %ls", *UniqueName);

	// Create the sandbox
	FCreateSandboxArgs Args;
	Args.Name = UniqueName;
	Args.Description = FString::Printf(TEXT("Created automatically on startup at %s"),
		*FDateTime::Now().ToString());

	if (SandboxModel->CreateNewSandbox(Args))
	{
		ShowSuccessNotification(FText::Format(
			LOCTEXT("Success.Created", "Created and joined sandbox '{0}'"),
			FText::FromString(UniqueName)));
	}
	else
	{
		FText ErrorReason;
		SandboxModel->CanCreateNewSandbox(UniqueName, &ErrorReason);
		UE_LOGF(LogSandboxedEditing, Error, "Failed to create sandbox '%ls': %ls",
			*UniqueName, *ErrorReason.ToString());
		ShowErrorNotification(FText::Format(
			LOCTEXT("Error.CreateFailed", "Failed to create sandbox: {0}"),
			ErrorReason));
	}
}

void FStartupBehaviorHandler::HandleJoinPreviousSandbox()
{
	UE_LOGF(LogSandboxedEditing, Log, "Startup behavior: Join Previous Sandbox");

	// Dismiss any active notification
	if (CurrentNotification.IsValid())
	{
		CurrentNotification->ExpireAndFadeout();
		CurrentNotification.Reset();
	}

	const USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();
	if (!Settings)
	{
		ShowErrorNotification(LOCTEXT("Error.NoSettings", "Failed to read settings"));
		return;
	}

	// Check if we have a last used sandbox path
	if (Settings->LastUsedSandboxPath.IsEmpty())
	{
		UE_LOGF(LogSandboxedEditing, Warning, "No previous sandbox found, creating new sandbox instead");
		HandleCreateNewSandbox();
		return;
	}

	// Check if sandbox still exists
	const TOptional<FSandboxInfo> InfoOptional = SandboxModel->GetSandboxInfo(Settings->LastUsedSandboxPath);
	if (!InfoOptional.IsSet())
	{
		UE_LOGF(LogSandboxedEditing, Warning, "Previous sandbox no longer exists at path: %ls, creating new sandbox instead",
			*Settings->LastUsedSandboxPath);

		// Clear stale path
		USandboxedEditingSettings* MutableSettings = USandboxedEditingSettings::Get();
		MutableSettings->LastUsedSandboxPath.Empty();
		MutableSettings->SaveConfig();

		// Fallback to creating a new sandbox
		HandleCreateNewSandbox();
		return;
	}

	const FSandboxInfo& Info = InfoOptional.GetValue();
	UE_LOGF(LogSandboxedEditing, Log, "Loading previous sandbox: %ls (path: %ls)",
		*Info.Name, *Info.SandboxRoot);

	// Load the sandbox
	if (SandboxModel->LoadSandbox(Info.SandboxRoot))
	{
		ShowSuccessNotification(FText::Format(
			LOCTEXT("Success.Loaded", "Loaded previous sandbox '{0}'"),
			FText::FromString(Info.Name)));
	}
	else
	{
		UE_LOGF(LogSandboxedEditing, Error, "Failed to load previous sandbox: %ls, creating new sandbox instead", *Info.Name);

		// Fallback to creating a new sandbox
		HandleCreateNewSandbox();
	}
}

FString FStartupBehaviorHandler::ResolveNamingTokens(const FString& Template)
{
	if (Template.IsEmpty())
	{
		UE_LOGF(LogSandboxedEditing, Verbose, "Empty template, using fallback");
		return FallbackToTimestamp();
	}

	// Check if GEngine is available
	if (GEngine == nullptr)
	{
		UE_LOGF(LogSandboxedEditing, Warning, "GEngine not available, using fallback naming");
		return FallbackToTimestamp();
	}

	// Get the NamingTokens subsystem
	UNamingTokensEngineSubsystem* Subsystem =
		GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();

	if (Subsystem == nullptr)
	{
		UE_LOGF(LogSandboxedEditing, Warning, "NamingTokens subsystem not available, using fallback naming");
		return FallbackToTimestamp();
	}

	// Evaluate the template
	FNamingTokenResultData Result = Subsystem->EvaluateTokenString(Template);
	FString EvaluatedName = Result.EvaluatedText.ToString();

	if (EvaluatedName.IsEmpty())
	{
		UE_LOGF(LogSandboxedEditing, Warning, "Token evaluation resulted in empty string, using fallback");
		return FallbackToTimestamp();
	}

	UE_LOGF(LogSandboxedEditing, Verbose, "Resolved naming tokens: '%ls' -> '%ls'",
		*Template, *EvaluatedName);

	return EvaluatedName;
}

FString FStartupBehaviorHandler::FallbackToTimestamp()
{
	FDateTime Now = FDateTime::Now();
	FString Fallback = FString::Printf(TEXT("Sandbox_%04d_%02d_%02d_%02d%02d"),
		Now.GetYear(), Now.GetMonth(), Now.GetDay(),
		Now.GetHour(), Now.GetMinute());

	UE_LOGF(LogSandboxedEditing, Verbose, "Using fallback timestamp name: %ls", *Fallback);
	return Fallback;
}

FString FStartupBehaviorHandler::GenerateUniqueSandboxName(const FString& BaseName)
{
	FText ErrorReason;

	// Try the base name first
	if (SandboxModel->CanCreateNewSandbox(BaseName, &ErrorReason))
	{
		return BaseName;
	}

	UE_LOGF(LogSandboxedEditing, Log, "Base name '%ls' already exists, trying auto-increment", *BaseName);

	// Try with suffixes: BaseName_1, BaseName_2, etc.
	for (int32 Suffix = 1; Suffix < 1000; ++Suffix)
	{
		FString Candidate = FString::Printf(TEXT("%s_%d"), *BaseName, Suffix);
		if (SandboxModel->CanCreateNewSandbox(Candidate, &ErrorReason))
		{
			UE_LOGF(LogSandboxedEditing, Log, "Found unique name with suffix: %ls", *Candidate);
			return Candidate;
		}
	}

	// Fallback to GUID for extreme edge case
	FString Unique = FString::Printf(TEXT("%s_%s"), *BaseName,
		*FGuid::NewGuid().ToString(EGuidFormats::Short));
	UE_LOGF(LogSandboxedEditing, Warning,
		"Generated GUID-based name after 1000 iterations: %ls", *Unique);
	return Unique;
}

void FStartupBehaviorHandler::ShowSuccessNotification(const FText& Message)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Info(Message);
	Info.bFireAndForget = true;
	Info.ExpireDuration = 3.0f;
	Info.bUseLargeFont = false;

	TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Info);
	if (Item.IsValid())
	{
		Item->SetCompletionState(SNotificationItem::CS_Success);
	}
}

void FStartupBehaviorHandler::ShowErrorNotification(const FText& Message)
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	FSlateNotificationManager& NotificationManager = FSlateNotificationManager::Get();
	FNotificationInfo Info(Message);
	Info.bFireAndForget = true;
	Info.ExpireDuration = 5.0f;
	Info.bUseLargeFont = false;

	TSharedPtr<SNotificationItem> Item = NotificationManager.AddNotification(Info);
	if (Item.IsValid())
	{
		Item->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

void FStartupBehaviorHandler::OnSandboxLoaded()
{
	if (!SandboxModel->HasActiveSandbox())
	{
		return;
	}

	FString CurrentPath = SandboxModel->GetActiveSandboxPath();
	USandboxedEditingSettings* Settings = USandboxedEditingSettings::Get();

	if (!Settings)
	{
		return;
	}

	// Update last used path if it changed
	if (Settings->LastUsedSandboxPath != CurrentPath)
	{
		Settings->LastUsedSandboxPath = CurrentPath;
		Settings->SaveConfig();
		UE_LOGF(LogSandboxedEditing, Verbose, "Updated last used sandbox path: %ls", *CurrentPath);
	}
}

} // namespace UE::SandboxedEditing

#undef LOCTEXT_NAMESPACE
