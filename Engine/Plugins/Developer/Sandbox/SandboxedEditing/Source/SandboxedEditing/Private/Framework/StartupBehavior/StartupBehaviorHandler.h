// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "SandboxedEditingSettings.h"
#include "Delegates/IDelegateInstance.h"

class SNotificationItem;

namespace UE::SandboxedEditing
{
class FSandboxSystemModel;

/**
 * Handles sandbox startup behavior based on user settings.
 * Automatically creates, loads, or prompts for sandboxes when the editor starts.
 * Also tracks the last used sandbox for the "Join Previous" workflow.
 */
class FStartupBehaviorHandler : public FNoncopyable
{
public:
	/**
	 * Constructor - registers for editor initialization and sandbox load events
	 * @param InModel The sandbox system model to use for operations
	 */
	explicit FStartupBehaviorHandler(const TSharedRef<FSandboxSystemModel>& InModel);

	/** Destructor - unregisters all delegates */
	~FStartupBehaviorHandler();

private:
	// Editor initialization
	/** Called when the editor has fully initialized and is ready */
	void OnEditorInitialized(double Duration);

	/** Reads settings and executes the appropriate startup behavior */
	void ExecuteStartupBehavior();

	// Behavior handlers
	/** Does nothing - default behavior */
	void HandleDoNothing();

	/** Shows a prompt with options to create new, join previous, or skip */
	void HandlePromptToJoin();

	/** Automatically creates a new sandbox using the template name */
	void HandleCreateNewSandbox();

	/** Automatically loads the most recently used sandbox */
	void HandleJoinPreviousSandbox();

	// Command line parsing
	/**
	 * Parses command line override for startup behavior
	 * @param OutBehavior If command line override is present, this will be set
	 * @return True if command line override was found, false otherwise
	 */
	bool TryGetStartupBehaviorFromCommandLine(ESandboxStartupBehavior& OutBehavior) const;

	/**
	 * Parses command line override for sandbox name template
	 * @param OutTemplate If command line override is present, this will be set
	 * @return True if command line override was found, false otherwise
	 */
	bool TryGetNameTemplateFromCommandLine(FString& OutTemplate) const;

	// Utilities
	/**
	 * Resolves naming tokens in a template string using the NamingTokens plugin
	 * @param Template The template string with tokens like {yyyy}, {mm}, {dd}, {user}
	 * @return The resolved string with tokens replaced, or a fallback if resolution fails
	 */
	FString ResolveNamingTokens(const FString& Template);

	/**
	 * Generates a fallback sandbox name using timestamp when NamingTokens fails
	 * @return A timestamp-based sandbox name like "Sandbox_2026_03_27_1435"
	 */
	FString FallbackToTimestamp();

	/**
	 * Generates a unique sandbox name by auto-incrementing if the base name exists
	 * @param BaseName The desired base name
	 * @return A unique name (BaseName, BaseName_1, BaseName_2, etc.)
	 */
	FString GenerateUniqueSandboxName(const FString& BaseName);

	/**
	 * Shows a success notification
	 * @param Message The message to display
	 */
	void ShowSuccessNotification(const FText& Message);

	/**
	 * Shows an error notification
	 * @param Message The error message to display
	 */
	void ShowErrorNotification(const FText& Message);

	// Last used tracking
	/**
	 * Called whenever a sandbox is loaded (manual or automatic)
	 * Updates LastUsedSandboxPath in settings
	 */
	void OnSandboxLoaded();

	// Members
	/** The sandbox system model for all sandbox operations */
	TSharedRef<FSandboxSystemModel> SandboxModel;

	/** Current notification being displayed (for prompt workflow) */
	TSharedPtr<SNotificationItem> CurrentNotification;

	/** Delegate handle for sandbox load tracking */
	FDelegateHandle LoadSandboxHandle;
};

} // namespace UE::SandboxedEditing
