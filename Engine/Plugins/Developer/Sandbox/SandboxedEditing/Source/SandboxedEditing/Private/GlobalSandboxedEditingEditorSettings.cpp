// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalSandboxedEditingEditorSettings.h"

#include "Misc/Paths.h"
#include "Misc/PathViews.h"

namespace UE::SandboxedEditing::GlobalSettingsDetail
{
static FString GetDirectoryPath(const FString& InPath)
{
	FString NormalizedPath = InPath;
	FPaths::NormalizeDirectoryName(NormalizedPath);
	const bool bIsDirectoryPath = FPaths::GetExtension(NormalizedPath).IsEmpty();
	return bIsDirectoryPath
		? NormalizedPath
		: FPaths::GetPath(NormalizedPath);
}
}

void UGlobalSandboxedEditingEditorSettings::SetDefaultExportDirectory(const FString& InPath)
{
	FString DirectoryPath = UE::SandboxedEditing::GlobalSettingsDetail::GetDirectoryPath(InPath);
	if (DirectoryPath != DefaultExportDirectory)
	{
		DefaultExportDirectory = MoveTemp(DirectoryPath);
		SaveConfig();
	}
}

void UGlobalSandboxedEditingEditorSettings::SetDefaultImportDirectory(const FString& InPath)
{
	FString DirectoryPath = UE::SandboxedEditing::GlobalSettingsDetail::GetDirectoryPath(InPath);
	if (DirectoryPath != DefaultImportDirectory)
	{
		DefaultImportDirectory = MoveTemp(DirectoryPath);
		SaveConfig();
	}
}
