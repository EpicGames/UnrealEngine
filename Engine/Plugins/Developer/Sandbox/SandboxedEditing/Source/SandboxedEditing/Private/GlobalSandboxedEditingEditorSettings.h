// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "GlobalSandboxedEditingEditorSettings.generated.h"

/** Settings shared with all editor instances. */
UCLASS(Config=EditorSettings, DefaultConfig)
class UGlobalSandboxedEditingEditorSettings : public UObject
{
	GENERATED_BODY()
public:
	
	static UGlobalSandboxedEditingEditorSettings* Get() { return GetMutableDefault<UGlobalSandboxedEditingEditorSettings>(); }

	/** @return The default path that sandboxes are exported to. */
	const FString& GetDefaultExportDirectory() const { return DefaultExportDirectory; }
	/** @param InPath Path to directory or file. In case of file, the directory will be extracted and set. */
	void SetDefaultExportDirectory(const FString& InPath);
	
	/** @return The default path that sandboxes are imported from. */
	const FString& GetDefaultImportDirectory() const { return DefaultImportDirectory; }
	/** @param InPath Path to directory or file. In case of file, the directory will be extracted and set. */
	void SetDefaultImportDirectory(const FString& InPath);
	
private:
	
	/** Default directory to export to. */
	UPROPERTY(EditAnywhere, Config, Category = "General")
	FString DefaultExportDirectory;
	
	/** Default directory to import from. */
	UPROPERTY(EditAnywhere, Config, Category = "General")
	FString DefaultImportDirectory;
};