// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxEditingModule.h"

#include "CoreGlobals.h"
#include "Features/Browser/Commands/BrowserCommands.h"
#include "Features/Browser/Commands/FileStateActions/FileStateCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Misc/CoreDelegates.h"
#include "SandboxedEditingSettings.h"
#include "SandboxedEditingStyle.h"

#define LOCTEXT_NAMESPACE "SandboxEditing"

namespace UE::SandboxedEditing
{
void RegisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings(
			"Project", "Plugins", "Sandbox",
			LOCTEXT("SandboxFileEditingSettings", "Sandbox Editing"),
			LOCTEXT("SandboxFileEditingSettingsDescription", "Configure the sandboxed editing settings."),
			USandboxedEditingSettings::Get());
	}
}
void UnregisterSettings()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Sandbox");
	}
}

void FSandboxEditingModule::StartupModule()
{
	FSandboxedEditingStyle::Get();
	
	FBrowserCommands::Register();
	FFileStateCommands::Register();
	
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]
	{
		if (FSlateApplication::IsInitialized() && GIsEditor)
		{
			EditingApp = MakeUnique<FSandboxedEditingApp>();
			RegisterSettings();
		}
	});
}

	
void FSandboxEditingModule::ShutdownModule()
{
	EditingApp.Reset();
	UnregisterSettings();
	
	FBrowserCommands::Unregister();
	FFileStateCommands::Unregister();
}
} // namespace UE::SandboxedEditing

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::SandboxedEditing::FSandboxEditingModule, SandboxedEditing);
