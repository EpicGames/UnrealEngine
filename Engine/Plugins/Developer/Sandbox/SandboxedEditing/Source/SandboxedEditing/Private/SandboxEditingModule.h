// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISandboxEditingModule.h"
#include "Framework/SandboxedEditingApp.h"

namespace UE::SandboxedEditing
{
class FSandboxEditingModule : public ISandboxedEditingModule
{
public:

	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

private:

	/** Manages all the business logic and creates the editor workflows. */
	TUniquePtr<FSandboxedEditingApp> EditingApp;
};
}

