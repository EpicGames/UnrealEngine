// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "SandboxedEditingSettings.generated.h"

/**
 * Determines how the editor should behave on startup with regards to sandboxes.
 *
 * Command Line Overrides:
 *   -SANDBOXSTARTUP=DoNothing|PromptToJoin|CreateNewSandbox|JoinPreviousSandbox
 *   -SANDBOXNAMETEMPLATE="Sandbox_{yyyy}_{mm}_{dd}"
 *
 * Command line overrides take precedence over these settings and are not saved to config.
 */
UENUM()
enum class ESandboxStartupBehavior : uint8
{
	/** Do not automatically create or join a sandbox on startup (default) */
	DoNothing UMETA(DisplayName = "Do Nothing"),

	/** Show a prompt asking the user if they want to create or join a sandbox */
	PromptToJoin UMETA(DisplayName = "Prompt to Join"),

	/** Automatically create a new sandbox using the default template name */
	CreateNewSandbox UMETA(DisplayName = "Create New Sandbox"),

	/** Automatically join the most recently used sandbox */
	JoinPreviousSandbox UMETA(DisplayName = "Join Previous Sandbox")
};

UCLASS(MinimalAPI, config=Engine)
class USandboxedEditingSettings : public UObject
{
	GENERATED_BODY()
public:

	static USandboxedEditingSettings* Get() { return GetMutableDefault<USandboxedEditingSettings>(); }

	DECLARE_MULTICAST_DELEGATE(FOnCustomDirectoryChanged);
	static FOnCustomDirectoryChanged OnCustomDirectoryChanged;

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	/** Should we always ask the user to persist sandbox contents when leaving a sandbox? */
	UPROPERTY(config, EditAnywhere, Category = "Actions", meta = (DisplayName="Ask to persist when leaving the sandbox"))
	bool bAskToPersistWhenLeavingSandbox = true;

	/** Controls what happens on editor startup with regards to sandboxes. Can be overridden with -SANDBOXSTARTUP=DoNothing|PromptToJoin|CreateNewSandbox|JoinPreviousSandbox */
	UPROPERTY(config, EditAnywhere, Category = "Startup", meta = (DisplayName = "Startup Behavior", ToolTip = "Controls what happens on editor startup. Can be overridden with -SANDBOXSTARTUP command line argument."))
	ESandboxStartupBehavior StartupBehavior = ESandboxStartupBehavior::DoNothing;

	/** Template for sandbox names when creating new sandboxes on startup. Supports {yyyy}, {mm}, {dd}, {24h}, {min}, {sec}, {user}, etc. Can be overridden with -SANDBOXNAMETEMPLATE */
	UPROPERTY(config, EditAnywhere, Category = "Startup", meta = (DisplayName = "Default Sandbox Name Template", EditCondition = "StartupBehavior != ESandboxStartupBehavior::DoNothing", EditConditionHides, ToolTip = "Template for sandbox names. Supports {yyyy}, {mm}, {dd}, {24h}, {min}, {sec}, {user}, etc. Example: Sandbox_{yyyy}_{mm}_{dd}_{user}. Can be overridden with -SANDBOXNAMETEMPLATE command line argument."))
	FString DefaultSandboxNameTemplate = TEXT("Sandbox_{yyyy}_{mm}_{dd}");

	/** Internal tracking of the last used sandbox path (auto-updated, not user-editable) */
	UPROPERTY(config)
	FString LastUsedSandboxPath;

	/** Automatically delete sandboxes with no file changes when the editor shuts down */
	UPROPERTY(config, EditAnywhere, Category = "Cleanup", meta = (DisplayName = "Delete Empty Sandboxes on Exit", ToolTip = "Automatically delete sandboxes with no file changes when the editor shuts down"))
	bool bDeleteEmptySandboxesOnExit = false;

	/** Optional custom directory where sandboxes are stored. Leave empty to use default location: {Project}/Intermediate/Sandboxes/. Can be overridden with -DEFAULTSANDBOXDIRECTORY command line argument. */
	UPROPERTY(config, EditAnywhere, Category = "Storage", meta = (DisplayName = "Custom Sandbox Storage Directory", ToolTip = "Optional custom directory where sandboxes are stored. Leave empty to use default location: {Project}/Intermediate/Sandboxes/. Can be overridden with -DEFAULTSANDBOXDIRECTORY command line argument."))
	FDirectoryPath CustomSandboxStorageDirectory;
};


