// Copyright Epic Games, Inc. All Rights Reserved.

#include "SandboxedEditingStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::SandboxedEditing
{
FSandboxedEditingStyle& FSandboxedEditingStyle::Get()
{
	static FSandboxedEditingStyle Style;
	return Style;
}

FSandboxedEditingStyle::FSandboxedEditingStyle()
	: FSlateStyleSet("SandboxedEditingStyle")
{
	const FString PluginContentDir = FPaths::EnginePluginsDir() / TEXT("Developer") / TEXT("Sandbox") / TEXT("SandboxedEditing") / TEXT("Content");
	const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate");
	SetContentRoot(PluginContentDir);
	SetCoreContentRoot(EngineEditorSlateDir);
	
	RegisterBrowserStyle();
	RegisterSharedStyle();
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSandboxedEditingStyle::~FSandboxedEditingStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FSandboxedEditingStyle::RegisterBrowserStyle()
{
	const FLinearColor IconColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
	const FVector2D Icon24x24(24.0f, 24.0f);
	
	// Icons
	Set("SandboxedEditing.Browser.Icon", new IMAGE_BRUSH_SVG("SandboxBrowserIcon_24", Icon24x24, IconColorAndOpacity));
	Set("SandboxedEditing.Browser.NewSandbox", new IMAGE_BRUSH_SVG("NewSandbox_24", Icon24x24, IconColorAndOpacity));
	Set("SandboxedEditing.Browser.DeleteSandbox", new IMAGE_BRUSH("DeleteSandbox_48", Icon24x24, IconColorAndOpacity));
	Set("SandboxedEditing.Browser.LeaveSandbox", new IMAGE_BRUSH("LeaveSandbox_48", Icon24x24, IconColorAndOpacity));
	Set("SandboxedEditing.Browser.ExportSandboxes", new FSlateBrush(*FAppStyle::Get().GetBrush("Icons.Export")));
	Set("SandboxedEditing.Browser.ImportSandboxes",  new FSlateBrush(*FAppStyle::Get().GetBrush("Icons.Import")));
	
	// Commands
	Set("Browser.ExportSandboxes", new FSlateBrush(*FAppStyle::Get().GetBrush("Icons.Export")));
	Set("Browser.ImportSandboxes", new FSlateBrush(*FAppStyle::Get().GetBrush("Icons.Import")));
	
	Set("FileState.BrowseToAsset", new FSlateBrush(*FAppStyle::Get().GetBrush("SystemWideCommands.FindInContentBrowser")));
	Set("FileState.ShowRootInExplorer", new FSlateBrush(*FAppStyle::Get().GetBrush("SystemWideCommands.FindInContentBrowser")));
	Set("FileState.ShowFileInExplorer", new FSlateBrush(*FAppStyle::Get().GetBrush("SystemWideCommands.FindInContentBrowser")));
	
	// Sandbox browser
	Set("SandboxedEditing.Browser.RowPadding", FMargin(0.f, 4.f));
	Set("SandboxedEditing.Browser.NameColumn.Padding", FMargin(16.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.Browser.NameColumn.FillWidth", 2.f);
	Set("SandboxedEditing.Browser.DescriptionColumn.Padding", FMargin(8.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.Browser.DescriptionColumn.FillWidth", 3.f);
	Set("SandboxedEditing.Browser.VersionColumn.Padding", FMargin(8.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.Browser.VersionColumn.FillWidth", 1.f);
	Set("SandboxedEditing.Browser.LastModifiedColumn.Padding", FMargin(8.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.Browser.LastModifiedColumn.FillWidth", 1.f);
	
	Set("SandboxedEditing.FileActions.RowPadding", FMargin(0.f, 4.f));
	Set("SandboxedEditing.FileActions.PathColumn.Padding", FMargin(8.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.FileActions.PathColumn.FillWidth", 2.f);
	Set("SandboxedEditing.FileActions.PersistCheckbox.Padding", FMargin(2.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.FileActions.PersistCheckbox.FixedWidth", 23.f);
	Set("SandboxedEditing.FileActions.FileActionColumn.Padding", FMargin(0.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.FileActions.FileActionColumn.FixedWidth", 23.f);
	Set("SandboxedEditing.FileActions.TimestampColumn.Padding", FMargin(8.f, 0.f, 0.f, 0.f));
	Set("SandboxedEditing.FileActions.TimestampColumn.FillWidth", 200.f);
}

void FSandboxedEditingStyle::RegisterSharedStyle()
{
	
}
}
