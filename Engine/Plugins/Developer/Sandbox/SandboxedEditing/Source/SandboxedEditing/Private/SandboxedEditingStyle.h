// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FSlateStyleSet;
class ISlateStyle;

namespace UE::SandboxedEditing
{
class FSandboxedEditingStyle final : public FSlateStyleSet
{
public:
	
	static FSandboxedEditingStyle& Get();

	FSandboxedEditingStyle();
	virtual ~FSandboxedEditingStyle() override;
	
private:
	
	void RegisterBrowserStyle();
	void RegisterSharedStyle();
};
}

