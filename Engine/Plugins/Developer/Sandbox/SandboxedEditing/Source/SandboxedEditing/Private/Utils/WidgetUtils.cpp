// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetUtils.h"

#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/SlateDelegates.h"
#include "ISettingsModule.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SNegativeActionButton.h"
#include "SPositiveActionButton.h"
#include "SPrimaryButton.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

namespace UE::SandboxedEditing
{
TSharedRef<SWidget> MakeIconButton(
	const TAttribute<const FSlateBrush*>& Icon,
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility
	)
{
	return SNew(SButton)
		.OnClicked(OnClicked)
		.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
		.ToolTipText(Tooltip)
		.Visibility(Visibility)
		.IsEnabled(EnabledAttribute)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image(Icon)
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

TSharedRef<SWidget> MakePrimaryButton(
	const TAttribute<FText>& Label,
	const TAttribute<FText>& Tooltip,
	const TAttribute<const FSlateBrush*>& Icon,
	const TAttribute<bool>& EnabledAttribute,
	const FOnClicked& OnClicked,
	const TAttribute<EVisibility>& Visibility
)
{
	return SNew(SPrimaryButton)
		.Icon(Icon)
		.ToolTipText(Tooltip)
		.IsEnabled(EnabledAttribute)
		.OnClicked(OnClicked)
		.Visibility(Visibility)
		.Text(Label);
}

TSharedRef<SWidget> MakeIconButtonByCommand(
	const TAttribute<const FSlateBrush*>& Icon,
	const TSharedPtr<FUICommandList>& InCommandList,
	const TSharedPtr<FUICommandInfo>& InCommand,
	const TAttribute<FText>& OverrideTooltip,
	const TAttribute<EVisibility>& Visibility
)
{
	return MakeIconButton(
		Icon, 
		OverrideTooltip.IsBound() || OverrideTooltip.IsSet() ? OverrideTooltip : InCommand->GetDescription(), 
		TAttribute<bool>::CreateLambda([InCommandList, InCommand]
		{
			const FUIAction* Action = InCommandList->GetActionForCommand(InCommand);
			return Action && Action->CanExecute();
		}),
		FOnClicked::CreateLambda([InCommandList, InCommand]
		{
			if (const FUIAction* Action = InCommandList->GetActionForCommand(InCommand))
			{
				Action->Execute();
			}
			return FReply::Handled();
		}), 
		Visibility
		);
}

TSharedRef<SWidget> MakePrimaryButtonByCommand(
	const TSharedPtr<FUICommandList>& InCommandList,
	const TSharedPtr<FUICommandInfo>& InCommand,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InOverrideTooltip,
	const TAttribute<const FSlateBrush*>& InIcon,
	const TAttribute<EVisibility>& Visibility
)
{
	return MakePrimaryButton(
		InLabel, 
		InOverrideTooltip.IsBound() || InOverrideTooltip.IsSet() ? InOverrideTooltip : InCommand->GetDescription(),
		InIcon, 
		TAttribute<bool>::CreateLambda([InCommandList, InCommand]
			{
				const FUIAction* Action = InCommandList->GetActionForCommand(InCommand);
				return Action && Action->CanExecute();
			}),
		FOnClicked::CreateLambda([InCommandList, InCommand]
			{
				if (const FUIAction* Action = InCommandList->GetActionForCommand(InCommand))
				{
					Action->Execute();
				}
				return FReply::Handled();
			}), 
		Visibility
	);
}

TSharedRef<SWidget> MakePositiveActionButton(
	const TAttribute<const FSlateBrush*>& Icon, 
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility
		)
{
	return SNew(SPositiveActionButton)
		.OnClicked(OnClicked)
		.ToolTipText(Tooltip)
		.Visibility(Visibility)
		.IsEnabled(EnabledAttribute)
		.Icon(Icon);
}

TSharedRef<SWidget> MakeNegativeActionButton(
	const TAttribute<const FSlateBrush*>& Icon, 
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility
		)
{
	return SNew(SNegativeActionButton)
		.OnClicked(OnClicked)
		.ToolTipText(Tooltip)
		.Visibility(Visibility)
		.IsEnabled(EnabledAttribute)
		.Icon(Icon);
}

TSharedRef<SWidget> CreateOpenSettingsButton()
{
	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
		.OnClicked_Lambda([]()
		{
			FModuleManager::GetModulePtr<ISettingsModule>("Settings")->ShowViewer("Project", "Plugins", "Sandbox"); 
			return FReply::Handled();
		})
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Settings"))
		];
}
}
