// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointerFwd.h"
#include "Layout/Visibility.h"

class FText;
class FUICommandInfo;
class FUICommandList;
class SWidget;
struct FSlateBrush;
template<typename ObjectType> class TAttribute;

namespace UE::SandboxedEditing
{
/** Makes an action button that only contains an icon. */
TSharedRef<SWidget> MakeIconButton(
	const TAttribute<const FSlateBrush*>& Icon,
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
	);

/** Makes primary action button with a label. */
TSharedRef<SWidget> MakePrimaryButton(
	const TAttribute<FText>& Label,
	const TAttribute<FText>& Tooltip,
	const TAttribute<const FSlateBrush*>& Icon,
	const TAttribute<bool>& EnabledAttribute,
	const FOnClicked& OnClicked,
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
);


/** Overload that determines the IsEnabled and OnClicked parameters from the command list. */
TSharedRef<SWidget> MakeIconButtonByCommand(
	const TAttribute<const FSlateBrush*>& Icon,
	const TSharedPtr<FUICommandList>& InCommandList,
	const TSharedPtr<FUICommandInfo>& InCommand,
	const TAttribute<FText>& OverrideTooltip = {},
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
);

/** Overload that determines the IsEnabled and OnClicked parameters from the command list. */
TSharedRef<SWidget> MakePrimaryButtonByCommand(
	const TSharedPtr<FUICommandList>& InCommandList,
	const TSharedPtr<FUICommandInfo>& InCommand,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InOverrideTooltip = {},
	const TAttribute<const FSlateBrush*>& InIcon = {},
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
);

/** Makes a positive action button that only contains an icon */
TSharedRef<SWidget> MakePositiveActionButton(
	const TAttribute<const FSlateBrush*>& Icon, 
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
	);

/** Makes a negative action button that only contains an icon */
TSharedRef<SWidget> MakeNegativeActionButton(
	const TAttribute<const FSlateBrush*>& Icon, 
	const TAttribute<FText>& Tooltip, 
	const TAttribute<bool>& EnabledAttribute, 
	const FOnClicked& OnClicked, 
	const TAttribute<EVisibility>& Visibility = EVisibility::Visible
	);

/** @return A gear button that opens the plugin settings for Sandboxed Editing. */
TSharedRef<SWidget> CreateOpenSettingsButton();
}
