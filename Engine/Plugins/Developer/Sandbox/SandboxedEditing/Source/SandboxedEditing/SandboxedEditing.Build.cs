// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SandboxedEditing : ModuleRules
	{
		public SandboxedEditing(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",  
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"ContentBrowser",
					"DesktopPlatform",
					"Engine",
					"FileSandboxCore",
					"FileSandboxUI",
					"FileUtilities",
					"InputCore",
					"NamingTokens",
					"Slate",
					"SlateCore",
					"SourceControl",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd",
					"WorkspaceMenuStructure",
				}
			);
		}
	}
}
